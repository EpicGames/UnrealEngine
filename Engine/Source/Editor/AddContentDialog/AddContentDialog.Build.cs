// Copyright Epic Games, Inc. All Rights Reserved.

using UnrealBuildTool;

[SupportedTargetTypes(TargetType.Editor, TargetType.Program)]
public class AddContentDialog : ModuleRules
{
	public AddContentDialog(ReadOnlyTargetRules Target) : base(Target)
	{
		PrivateIncludePathModuleNames.AddRange(
			new string[] {
				"AssetTools",
				"ContentBrowser"
			}
		);

		PrivateDependencyModuleNames.AddRange(
			new string[] {
				"ApplicationCore",
				"Core",
				"CoreUObject",
				"DesktopPlatform",
				"DirectoryWatcher",
				"EditorFramework",
				"Engine",
				"ImageWrapper",
				"InputCore",
				"Json",
				"PakFile",
				"Slate",
				"SlateCore",
				"ToolWidgets",
				"UnrealEd",
				"WidgetCarousel"
			}
		);
	}
}
