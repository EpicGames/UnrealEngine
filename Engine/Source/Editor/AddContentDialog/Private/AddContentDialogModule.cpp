// Copyright Epic Games, Inc. All Rights Reserved.

#include "ContentSourceProviderManager.h"
#include "ContentSourceProviders/FeaturePack/FeaturePackContentSourceProvider.h"
#include "Framework/Application/SlateApplication.h"
#include "IAddContentDialogModule.h"
#include "Modules/ModuleManager.h"
#include "SAddContentDialog.h"
#include "Templates/SharedPointer.h"
#include "WidgetCarouselModule.h"
#include "WidgetCarouselStyle.h"
#include "Widgets/DeclarativeSyntaxSupport.h"

class SWindow;

#define LOCTEXT_NAMESPACE "AddContentDialog"

class FAddContentDialogModule : public IAddContentDialogModule
{

public:
	virtual void StartupModule() override
	{
		FModuleManager::LoadModuleChecked<FWidgetCarouselModule>("WidgetCarousel");
		FWidgetCarouselModuleStyle::Initialize();

		ContentSourceProviderManager = TSharedPtr<FContentSourceProviderManager>(new FContentSourceProviderManager());
		ContentSourceProviderManager->RegisterContentSourceProvider(MakeShareable(new FFeaturePackContentSourceProvider()));
	}

	virtual void ShutdownModule() override
	{
		FWidgetCarouselModuleStyle::Shutdown();
	}

	virtual TSharedRef<FContentSourceProviderManager> GetContentSourceProviderManager() override
	{
		return ContentSourceProviderManager.ToSharedRef();
	}

	virtual void ShowDialog(TSharedRef<SWindow> ParentWindow) override
	{
		if (AddContentDialog.IsValid() == false)
		{
			TSharedRef<SWindow> Dialog = SNew(SAddContentDialog);
			FSlateApplication::Get().AddWindowAsNativeChild(Dialog, ParentWindow);
			AddContentDialog = TWeakPtr<SWindow>(Dialog);
		}		
	}

private:
	TSharedPtr<FContentSourceProviderManager> ContentSourceProviderManager;
	TWeakPtr<SWindow> AddContentDialog;
};

IMPLEMENT_MODULE(FAddContentDialogModule, AddContentDialog);

#undef LOCTEXT_NAMESPACE
