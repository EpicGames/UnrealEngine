// Copyright Epic Games, Inc. All Rights Reserved.

#include "ContentSourceDragDropOp.h"

#include "GenericPlatform/ICursor.h"
#include "HAL/PlatformCrt.h"
#include "Misc/Attribute.h"
#include "Misc/Optional.h"
#include "ViewModels/ContentSourceViewModel.h"
#include "Widgets/DeclarativeSyntaxSupport.h"
#include "Widgets/Images/SImage.h"

class SWidget;

TSharedRef<FContentSourceDragDropOp> FContentSourceDragDropOp::CreateShared(TSharedPtr<FContentSourceViewModel> InContentSource)
{
	TSharedPtr<FContentSourceDragDropOp> DragDropOp = MakeShareable(new FContentSourceDragDropOp(InContentSource));
	DragDropOp->MouseCursor = EMouseCursor::GrabHandClosed;
	DragDropOp->Construct();
	return DragDropOp.ToSharedRef();
}

FContentSourceDragDropOp::FContentSourceDragDropOp(TSharedPtr<FContentSourceViewModel> InContentSource)
{
	ContentSource = InContentSource;
}

TSharedPtr<SWidget> FContentSourceDragDropOp::GetDefaultDecorator() const
{
	return SNew(SImage)
		.Image(ContentSource->GetIconBrush().Get());
}

TSharedPtr<FContentSourceViewModel> FContentSourceDragDropOp::GetContentSource()
{
	return ContentSource;
}
