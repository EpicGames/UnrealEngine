// Copyright Epic Games, Inc. All Rights Reserved.

#pragma once

#include "Containers/UnrealString.h"
#include "DragAndDrop/DecoratedDragDropOp.h"
#include "Input/DragAndDrop.h"
#include "Templates/SharedPointer.h"

class FContentSourceViewModel;
class SWidget;

/** A drag drop operation for dragging and dropping FContentSourceViewModels. */
class FContentSourceDragDropOp : public FDecoratedDragDropOp
{
public:
	DRAG_DROP_OPERATOR_TYPE(FContentSourceDragDropOp, FDecoratedDragDropOp)

	/** Creates and constructs a shared references to a FContentSourceDragDrop.
		@param InContentSource - The view model for the content source being dragged and dropped */
	static TSharedRef<FContentSourceDragDropOp> CreateShared(TSharedPtr<FContentSourceViewModel> InContentSource);

	virtual TSharedPtr<SWidget> GetDefaultDecorator() const override;

	/** Gets the view model for the content source being dragged and dropped. */
	TSharedPtr<FContentSourceViewModel> GetContentSource();

private:
	FContentSourceDragDropOp(TSharedPtr<FContentSourceViewModel> InContentSource);

private:
	/** The view model for the content source being dragged and dropped. */
	TSharedPtr<FContentSourceViewModel> ContentSource;
};
