// Copyright Epic Games, Inc. All Rights Reserved.

#include "ContentSourceProviderManager.h"

#include "HAL/PlatformCrt.h"

class IContentSourceProvider;

void FContentSourceProviderManager::RegisterContentSourceProvider(TSharedRef<IContentSourceProvider> ContentSourceProvider)
{
	ContentSourceProviders.Add(ContentSourceProvider);
}

const TArray<TSharedRef<IContentSourceProvider>>* FContentSourceProviderManager::GetContentSourceProviders()
{
	return &ContentSourceProviders;
}
