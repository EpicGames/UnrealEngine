// Copyright Epic Games, Inc. All Rights Reserved.

#pragma once

#include "Containers/Array.h"
#include "Templates/SharedPointer.h"

class IContentSourceProvider;

/** Provides methods for registering and getting registered content source providers. */
class FContentSourceProviderManager
{
public:
	/** Registers a content source provider. */
	void RegisterContentSourceProvider(TSharedRef<IContentSourceProvider> ContentSourceProvider);

	/** Gets and array of the currently registered content source providers. */
	const TArray<TSharedRef<IContentSourceProvider>>* GetContentSourceProviders();

private:
	/** The registered content source providers. */
	TArray<TSharedRef<IContentSourceProvider>> ContentSourceProviders;
};
