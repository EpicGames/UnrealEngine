// Copyright Epic Games, Inc. All Rights Reserved.

#include "FeaturePackContentSource.h"

#include "AssetCompilingManager.h"
#include "AssetToolsModule.h"
#include "Containers/Map.h"
#include "Containers/StringFwd.h"
#include "Containers/StringView.h"
#include "ContentBrowserModule.h"
#include "ContentSourceProviderManager.h"
#include "CoreGlobals.h"
#include "Dom/JsonObject.h"
#include "Dom/JsonValue.h"
#include "FileHelpers.h"
#include "GenericPlatform/GenericPlatformFile.h"
#include "HAL/FileManager.h"
#include "HAL/PlatformFileManager.h"
#include "IAddContentDialogModule.h"
#include "IAssetTools.h"
#include "IContentBrowserSingleton.h"
#include "IContentSource.h"
#include "IContentSourceProvider.h"
#include "IPlatformFilePak.h"
#include "Logging/LogCategory.h"
#include "Logging/LogMacros.h"
#include "Misc/ConfigCacheIni.h"
#include "Misc/FileHelper.h"
#include "Misc/Paths.h"
#include "Modules/ModuleManager.h"
#include "Serialization/JsonReader.h"
#include "Serialization/JsonSerializer.h"
#include "Serialization/JsonTypes.h"
#include "Templates/Tuple.h"
#include "Templates/UniquePtr.h"
#include "Trace/Detail/Channel.h"
#include "UObject/NameTypes.h"
#include "UObject/Object.h"
#include "UObject/UObjectGlobals.h"

#include UE_INLINE_GENERATED_CPP_BY_NAME(FeaturePackContentSource)

class UPackage;

#define LOCTEXT_NAMESPACE "ContentFeaturePacks"

DEFINE_LOG_CATEGORY_STATIC(LogFeaturePack, Log, All);

bool TryValidateTranslatedValue(TSharedPtr<FJsonValue> TranslatedValue, TSharedPtr<FString>& ErrorMessage)
{
	if (TranslatedValue.IsValid() == false)
	{
		ErrorMessage = MakeShareable(new FString("Invalid translated value"));
		return false;
	}

	const TSharedPtr<FJsonObject>* TranslatedObject;
	if (TranslatedValue->TryGetObject(TranslatedObject) == false)
	{
		ErrorMessage = MakeShareable(new FString("Invalid translated value"));
		return false;
	}

	if ((*TranslatedObject)->HasTypedField<EJson::String>(TEXT("Language")) == false)
	{
		ErrorMessage = MakeShareable(new FString("Translated value missing 'Language' field"));
		return false;
	}

	if ((*TranslatedObject)->HasTypedField<EJson::String>(TEXT("Text")) == false)
	{
		ErrorMessage = MakeShareable(new FString("Translated value missing 'Text' field"));
		return false;
	}

	return true;
}

bool TryValidateManifestObject(TSharedPtr<FJsonObject> ManifestObject, TSharedPtr<FString>& ErrorMessage)
{
	if (ManifestObject.IsValid() == false)
	{
		ErrorMessage = MakeShareable(new FString("Manifest object missing"));
		return false;
	}

	if (ManifestObject->HasTypedField<EJson::Array>(TEXT("Name")) == false)
	{
		ErrorMessage = MakeShareable(new FString("Manifest object missing 'Names' field"));
		return false;
	}

	for (TSharedPtr<FJsonValue> NameValue : ManifestObject->GetArrayField(TEXT("Name")))
	{
		if (TryValidateTranslatedValue(NameValue, ErrorMessage) == false)
		{
			ErrorMessage = MakeShareable(new FString(FString::Printf(TEXT("Manifest object 'Names' field error: %s"), **ErrorMessage)));
		}
	}

	if (ManifestObject->HasTypedField<EJson::Array>(TEXT("Description")) == false)
	{
		ErrorMessage = MakeShareable(new FString("Manifest object missing 'Description' field"));
		return false;
	}

	for (TSharedPtr<FJsonValue> DescriptionValue : ManifestObject->GetArrayField(TEXT("Description")))
	{
		if (TryValidateTranslatedValue(DescriptionValue, ErrorMessage) == false)
		{
			ErrorMessage = MakeShareable(new FString(FString::Printf(TEXT("Manifest object 'Description' field error: %s"), **ErrorMessage)));
		}
	}

	if (ManifestObject->HasTypedField<EJson::Array>(TEXT("AssetTypes")) == false)
	{
		ErrorMessage = MakeShareable(new FString("Manifest object missing 'AssetTypes' field"));
		return false;
	}

	for (TSharedPtr<FJsonValue> AssetTypesValue : ManifestObject->GetArrayField(TEXT("AssetTypes")))
	{
		if (TryValidateTranslatedValue(AssetTypesValue, ErrorMessage) == false)
		{
			ErrorMessage = MakeShareable(new FString(FString::Printf(TEXT("Manifest object 'AssetTypes' field error: %s"), **ErrorMessage)));
		}
	}

	if (ManifestObject->HasTypedField<EJson::String>(TEXT("ClassTypes")) == false)
	{
		ErrorMessage = MakeShareable(new FString("Manifest object missing 'ClassTypes' field"));
		return false;
	}

	if (ManifestObject->HasTypedField<EJson::Array>(TEXT("Category")) == false && ManifestObject->HasTypedField<EJson::String>(TEXT("Category")) == false)
	{
		ErrorMessage = MakeShareable(new FString("Manifest object missing 'Category' field"));
		return false;
	}
		
	if (ManifestObject->HasTypedField<EJson::String>(TEXT("Thumbnail")) == false)
	{
		ErrorMessage = MakeShareable(new FString("Manifest object missing 'Thumbnail' field"));
		return false;
	}

	if (ManifestObject->HasTypedField<EJson::Array>(TEXT("Screenshots")) == false)
	{
		ErrorMessage = MakeShareable(new FString("Manifest object missing 'Screenshots' field"));
		return false;
	}

	// If we have an additional files entry check its valid
	if (ManifestObject->HasTypedField<EJson::Object>(TEXT("AdditionalFiles")) == true)
	{
		TSharedPtr<FJsonObject> AdditionalFileObject = ManifestObject->GetObjectField(TEXT("AdditionalFiles"));
		if (AdditionalFileObject->HasTypedField<EJson::String>(TEXT("DestinationFilesFolder")) == false )
		{
			ErrorMessage = MakeShareable(new FString("Manifest has an AdditionalFiles object but no DestinationFilesFolder"));
			return false;
		}
		if (AdditionalFileObject->HasTypedField<EJson::Array>(TEXT("AdditionalFilesList")) == false)
		{
			ErrorMessage = MakeShareable(new FString("Manifest has an AdditionalFiles object but no AdditionalFilesList"));
			return false;
		}
	}
	return true;
}

FFeaturePackContentSource::FFeaturePackContentSource(FString InFeaturePackPath)
{
	FeaturePackPath = InFeaturePackPath;
	bPackValid = false;
	
	FString ManifestString;
	Categories = { EContentSourceCategory::Unknown };
	if( InFeaturePackPath.EndsWith(TEXT(".upack") ) == true )
	{
		bContentsInPakFile = true;
		MountPoint = FPaths::GameFeatureRootPrefix();

		const FString PakFileName = TEXT("PakFile");
		FPakPlatformFile* PakPlatformFile = static_cast<FPakPlatformFile*>(FPlatformFileManager::Get().FindPlatformFile(*PakFileName));
		if (!PakPlatformFile)
		{
			// Create a pak platform file and mount the feature pack file.
			PakPlatformFile = static_cast<FPakPlatformFile*>(FPlatformFileManager::Get().GetPlatformFile(*PakFileName));
			PakPlatformFile->Initialize(&FPlatformFileManager::Get().GetPlatformFile(), TEXT(""));
		}
		PakPlatformFile->Mount(*InFeaturePackPath, 0, *MountPoint);

		// Gets the manifest file as a JSon string
		TArray<uint8> ManifestBuffer;
		if (LoadPakFileToBuffer(*PakPlatformFile, FPaths::Combine(*MountPoint, TEXT("manifest.json")), ManifestBuffer) == false)
		{
			RecordAndLogError(FString::Printf(TEXT("Error in Feature pack %s. Cannot find manifest."), *FeaturePackPath));
			return;
		}
		
		FFileHelper::BufferToString(ManifestString, ManifestBuffer.GetData(), ManifestBuffer.Num());

		if (ParseManifestString(ManifestString) == true)
		{
			LoadFeaturePackImageDataFromPackFile(*PakPlatformFile);			
		}
		PakPlatformFile->Unmount(*InFeaturePackPath);
	}
	else
	{
		bContentsInPakFile = false;
		FString TemplatesFolder = TEXT("FeaturePack");
		FString ThisTemplateRoot = FPaths::GetPath(FeaturePackPath);
		if( ThisTemplateRoot.EndsWith(TemplatesFolder) == true )
		{
			int32 Index = ThisTemplateRoot.Find(TemplatesFolder);
			ThisTemplateRoot.LeftInline(Index);
		}
		MountPoint = ThisTemplateRoot;
		FFileHelper::LoadFileToString( ManifestString, *FeaturePackPath);
		if (ParseManifestString(ManifestString) == true)
		{
			LoadFeaturePackImageData();			
		}
	}
}

FFeaturePackContentSource::FFeaturePackContentSource()
{
	bPackValid = false;
}

bool FFeaturePackContentSource::LoadPakFileToBuffer(FPakPlatformFile& PakPlatformFile, FString Path, TArray<uint8>& Buffer)
{
	bool bResult = false;
	TSharedPtr<IFileHandle> FileHandle(PakPlatformFile.OpenRead(*Path));
	if( FileHandle.IsValid())
	{
		Buffer.AddUninitialized(FileHandle->Size());
		bResult = FileHandle->Read(Buffer.GetData(), FileHandle->Size());
	}
	return bResult;
}

const TArray<FLocalizedText>& FFeaturePackContentSource::GetLocalizedNames() const
{
	return LocalizedNames;
}

const TArray<FLocalizedText>& FFeaturePackContentSource::GetLocalizedDescriptions() const
{
	return LocalizedDescriptions;
}

const TArray<FLocalizedText>& FFeaturePackContentSource::GetLocalizedAssetTypes() const
{
	return LocalizedAssetTypesList;
}


const FString& FFeaturePackContentSource::GetClassTypesUsed() const
{
	return ClassTypes;
}

const TArray<EContentSourceCategory>& FFeaturePackContentSource::GetCategories() const
{
	return Categories;
}

TSharedPtr<FImageData> FFeaturePackContentSource::GetIconData() const
{
	return IconData;
}

const TArray<TSharedPtr<FImageData>>& FFeaturePackContentSource::GetScreenshotData() const
{
	return ScreenshotData;
}

bool FFeaturePackContentSource::InstallToProject(FString InstallPath)
{
	bool bResult = false;
	if( IsDataValid() == false )
	{
		UE_LOGF(LogFeaturePack, Warning, "Trying to install invalid pack %ls", *InstallPath);
	}
	else
	{
		// We need to insert additional packs before we import the main assets since the code in the main pack may reference them
		// TODO - handle errors from this
		TArray<FString> FilesCopied;
		InsertAdditionalResources(AdditionalFeaturePacks,EFeaturePackDetailLevel::High, FPaths::ProjectDir(),FilesCopied);

 		if (AdditionalFilesForPack.AdditionalFilesList.Num() != 0)
 		{
 			bool bHasSourceFiles = false;
 			CopyAdditionalFilesToFolder( FPaths::ProjectDir(), FilesCopied, bHasSourceFiles/*,AdditionalFilesFolder*/);
 		}

		if( bContentsInPakFile == true)
		{
			FAssetToolsModule& AssetToolsModule = FModuleManager::Get().LoadModuleChecked<FAssetToolsModule>("AssetTools");
			TArray<FString> AssetPaths;
			AssetPaths.Add(FeaturePackPath);
			TArray<UObject*> ImportedObjects = AssetToolsModule.Get().ImportAssets(AssetPaths, InstallPath);

			if (ImportedObjects.Num() == 0)
			{
				UE_LOGF(LogFeaturePack, Warning, "No objects imported installing pack %ls", *InstallPath);
			}
			else
			{
				// Save any imported assets.
				TArray<UPackage*> ToSave;
				for (UObject* ImportedObject : ImportedObjects)
				{
					ToSave.AddUnique(ImportedObject->GetOutermost());
				}
				FEditorFileUtils::PromptForCheckoutAndSave(ToSave, /*bCheckDirty=*/ false, /*bPromptToSave=*/ false);

				bResult = true;
			}
		}
		
		// Focus on a specific asset if we want to.
		if (GetFocusAssetName().IsEmpty() == false)
		{
			UObject* FocusAsset = LoadObject<UObject>(nullptr, *GetFocusAssetName());
			if (FocusAsset)
			{
				FContentBrowserModule& ContentBrowserModule = FModuleManager::Get().LoadModuleChecked<FContentBrowserModule>("ContentBrowser");
				TArray<UObject*> SyncObjects;
				SyncObjects.Add(FocusAsset);
				ContentBrowserModule.Get().SyncBrowserToAssets(SyncObjects);
			}
		}

	}
	return bResult;
}

FFeaturePackContentSource::~FFeaturePackContentSource()
{
}

bool FFeaturePackContentSource::IsDataValid() const
{
	if( bPackValid == false )
	{
		return false;
	}
	// To Do maybe validate other data here
	return true;	
}

const FString& FFeaturePackContentSource::GetFocusAssetName() const
{
	return FocusAssetIdent;
}

const FString& FFeaturePackContentSource::GetSortKey() const
{
	return SortKey;
}

const FString& FFeaturePackContentSource::GetIdent() const
{
	return Identity;
}

FLocalizedText FFeaturePackContentSource::ChooseLocalizedText(TArray<FLocalizedText> Choices, FString LanguageCode)
{
	FLocalizedText Default;
	for (const FLocalizedText& Choice : Choices)
	{
		if (Choice.GetTwoLetterLanguage() == LanguageCode)
		{
			return Choice;
		}
		else if (Choice.GetTwoLetterLanguage() == TEXT("en"))
		{
			Default = Choice;
		}
	}
	return Default;
}

FLocalizedTextArray FFeaturePackContentSource::ChooseLocalizedTextArray(TArray<FLocalizedTextArray> Choices, FString LanguageCode)
{
	FLocalizedTextArray Default;
	for (const FLocalizedTextArray& Choice : Choices)
	{
		if (Choice.GetTwoLetterLanguage() == LanguageCode)
		{
			return Choice;
		}
		else if (Choice.GetTwoLetterLanguage() == TEXT("en"))
		{
			Default = Choice;
		}
	}
	return Default;
}

bool FFeaturePackContentSource::GetAdditionalFilesForPack(TArray<FString>& FileList, bool& bContainsSource)
{
	bool bParsedFiles = false;
	if (bPackValid == false)
	{
		RecordAndLogError(FString::Printf(TEXT("Cannot extract files from invalid Pack %s"), *FeaturePackPath));
	}
	else
	{
		// This doesnt support additional files in the manifest and in the config file. Should only do one or the other.
		if( AdditionalFilesForPack.AdditionalFilesList.Num() != 0 )
		{
			BuildListOfAdditionalFiles(AdditionalFilesForPack.AdditionalFilesList,FileList,bContainsSource);
		}
		else if( bContentsInPakFile == true)
		{
			// Create a pak platform file and mount the feature pack file.
			FPakPlatformFile PakPlatformFile;
			PakPlatformFile.Initialize(&FPlatformFileManager::Get().GetPlatformFile(), TEXT(""));			
			PakPlatformFile.Mount(*FeaturePackPath, 0, *MountPoint);

			// Gets the manifest file as a JSon string
			TArray<uint8> ManifestBuffer;
			if (LoadPakFileToBuffer(PakPlatformFile, FPaths::Combine(*MountPoint, TEXT("Config/Config.ini")), ManifestBuffer) == false)
			{
				RecordAndLogError(FString::Printf(TEXT("Error in Feature pack %s. Cannot find Config.ini"), *FeaturePackPath));
			}
			else
			{
				FString ConfigFileString;
				FFileHelper::BufferToString(ConfigFileString, ManifestBuffer.GetData(), ManifestBuffer.Num());
				bParsedFiles = ExtractListOfAdditionalFiles(ConfigFileString, FileList, bContainsSource);
			}
		}
	}
	return bParsedFiles;
}

bool FFeaturePackContentSource::ExtractListOfAdditionalFiles(const FString& InConfigFileAsString,TArray<FString>& InFileList, bool& bContainsSource)
{
	FConfigFile PackConfig;
	PackConfig.ProcessInputFileContents(InConfigFileAsString, TEXT("Uknown, see FFeaturePackContentSource::ExtractListOfAdditionalFiles"));
	const FConfigSection* AdditionalFilesSection = PackConfig.FindSection("AdditionalFilesToAdd");
	
	bContainsSource = false;
	bool bParsedAdditionFiles = false;
	if (AdditionalFilesSection)
	{
		TArray<FString> AdditionalFilesMap;
		
		bParsedAdditionFiles = true;
		for (auto& FilePair : *AdditionalFilesSection)
		{
			if (FilePair.Key.ToString().Contains("Files"))
			{
				AdditionalFilesMap.Add(FilePair.Value.GetValue());
			}
		}
		BuildListOfAdditionalFiles(AdditionalFilesMap,InFileList,bContainsSource);
				
	}
	return bParsedAdditionFiles;
}

void FFeaturePackContentSource::BuildListOfAdditionalFiles(TArray<FString>& AdditionalFileSourceList, TArray<FString>& FileList, bool& bContainsSourceFiles)
{
	for (const FString& FileSource : AdditionalFileSourceList)
	{
		FString Filename = FPaths::GetCleanFilename(FileSource);
		FString Directory = FPaths::RootDir() / FPaths::GetPath(FileSource);
		FPaths::MakeStandardFilename(Directory);
		FPakFile::MakeDirectoryFromPath(Directory);

		if (Filename.Contains(TEXT("*")))
		{
			TArray<FString> FoundFiles;
			IFileManager::Get().FindFilesRecursive(FoundFiles, *Directory, *Filename, true, false);
			FileList.Append(FoundFiles);
			if (!bContainsSourceFiles)
			{
				for (const FString& FoundFile : FoundFiles)
				{
					if (FoundFile.StartsWith(TEXT("Source/")) || FoundFile.Contains(TEXT("/Source/")))
					{
						bContainsSourceFiles = true;
						break;
					}
				}
			}
		}
		else
		{
			FileList.Add(Directory / Filename);
			if (!bContainsSourceFiles && (FileList.Last().StartsWith(TEXT("Source/")) || FileList.Last().Contains(TEXT("/Source/"))))
			{
				bContainsSourceFiles = true;
			}
		}
	}
}

void FFeaturePackContentSource::ImportPendingPacks()
{ 
	bool bAddPacks;
	if (GConfig->GetBool(TEXT("StartupActions"), TEXT("bAddPacks"), bAddPacks, GGameIni) == true)
	{
		if (bAddPacks == true)
		{
			ParseAndImportPacks();			
			GConfig->SetBool(TEXT("StartupActions"), TEXT("bAddPacks"), false, GGameIni);
			GConfig->Flush(true, GGameIni);
		}
	}
}

void FFeaturePackContentSource::ParseAndImportPacks()
{
	FAssetToolsModule& AssetToolsModule = FModuleManager::Get().LoadModuleChecked<FAssetToolsModule>("AssetTools");
	// Look for pack insertions in the startup actions section
	TArray<FString> PacksToAdd;
	GConfig->GetArray(TEXT("StartupActions"), TEXT("InsertPack"), PacksToAdd, GGameIni);
	int32 PacksInserted = 0;
	for (int32 iPackEntry = 0; iPackEntry < PacksToAdd.Num(); iPackEntry++)
	{
		FPackData EachPackData;
		TArray<FString> PackEntries;
		PacksToAdd[iPackEntry].ParseIntoArray(PackEntries, TEXT(","), true);
		FString PackSource;
		FString PackName;
		// Parse the pack name and source
		for (int32 iEntry = 0; iEntry < PackEntries.Num(); iEntry++)
		{
			FString EachString = PackEntries[iEntry];
			// remove the parenthesis
			EachString = EachString.Replace(TEXT("("), TEXT(""));
			EachString = EachString.Replace(TEXT(")"), TEXT(""));
			if (EachString.StartsWith(TEXT("PackSource=")) == true)
			{
				EachString = EachString.Replace(TEXT("PackSource="), TEXT(""));
				EachString = EachString.TrimQuotes();
				EachPackData.PackSource = EachString;
			}
			if (EachString.StartsWith(TEXT("PackName=")) == true)
			{
				EachString = EachString.Replace(TEXT("PackName="), TEXT(""));
				EachString = EachString.TrimQuotes();
				EachPackData.PackName = EachString;
			}
		}

		// If we found anything to insert, insert it !
		if ((EachPackData.PackSource.IsEmpty() == false) && (EachPackData.PackName.IsEmpty() == false))
		{
			TArray<FString> EachImport;
			FString FullPath = FPaths::FeaturePackDir() + EachPackData.PackSource;
			EachImport.Add(FullPath);
			EachPackData.ImportedObjects = AssetToolsModule.Get().ImportAssets(EachImport, TEXT("/Game"), nullptr, false);

			if (EachPackData.ImportedObjects.Num() == 0)
			{
				UE_LOGF(LogFeaturePack, Warning, "No objects imported installing pack %ls", *EachPackData.PackSource);
			}
			else
			{
				// Save any imported assets.
				TArray<UPackage*> ToSave;
				for (UObject* ImportedObject : EachPackData.ImportedObjects)
				{
					ToSave.AddUnique(ImportedObject->GetOutermost());
				}

				// Make sure any async compilation kicked off during ImportAssets is completed before we save.
				FAssetCompilingManager::Get().FinishAllCompilation();

				FEditorFileUtils::PromptForCheckoutAndSave(ToSave, /*bCheckDirty=*/ false, /*bPromptToSave=*/ false);
				PacksInserted++;
			}
		}
	}
	UE_LOGF(LogFeaturePack, Log, "Inserted %d feature packs", PacksInserted);
}

void FFeaturePackContentSource::RecordAndLogError(const FString& ErrorString)
{
	UE_LOGF(LogFeaturePack, Error, "%ls", *ErrorString);
	ParseErrors.Add(ErrorString);
}

void FFeaturePackContentSource::CopyAdditionalFilesToFolder( const FString& DestinationFolder, TArray<FString>& FilesCopied, bool &bHasSourceFiles, FString InGameFolder )
{
	// Array of pairs - possible folder identifier and bool value indicating whether Content folder should be appended to BaseFolderName
	// For example, we want to move contents of __ExternalActors__/ folder to Content/__ExternalActors__/
	TStaticArray<TPair<FString, bool>, 3> FolderIdents;
	FolderIdents[0] = { TEXT("__ExternalActors__/"), true };
	FolderIdents[1] = { TEXT("__ExternalObjects__/"), true };
	FolderIdents[2] = { TEXT("Content/"), false };

	TArray<FString> FilesToAdd;
	GetAdditionalFilesForPack(FilesToAdd, bHasSourceFiles);
	for (int32 iFile = 0; iFile < FilesToAdd.Num(); ++iFile)
	{
		FString EachFile = FilesToAdd[iFile];

		int32 BaseFolderNameIndex = INDEX_NONE;
		int32 BaseFolderNameLength = 0;
		bool bAppendContentFolderToBaseFolderName = false;

		for (const TPair<FString, bool>& FolderIdentPair : FolderIdents)
		{
			if (int32 FolderNameIndex = EachFile.Find(FolderIdentPair.Key); FolderNameIndex != INDEX_NONE)
			{
				BaseFolderNameIndex = FolderNameIndex;
				BaseFolderNameLength = FolderIdentPair.Key.Len();
				bAppendContentFolderToBaseFolderName = FolderIdentPair.Value;
				break;
			}
		}

		if (BaseFolderNameIndex != INDEX_NONE)
		{
			FString ContentFile = EachFile.RightChop(BaseFolderNameIndex);
			FString GameFolder = InGameFolder;
			if(( GameFolder.IsEmpty() == false) && ( GameFolder.StartsWith(TEXT("/")) == false ))
			{
				GameFolder.InsertAt(0,TEXT("/"));
			}
			FPaths::NormalizeFilename(ContentFile);
			if( GameFolder.IsEmpty() == false)
			{
				ContentFile.InsertAt(BaseFolderNameLength - 1, GameFolder);
			}

			if (bAppendContentFolderToBaseFolderName)
			{
				ContentFile = TEXT("Content/") + ContentFile;
			}

			FString FinalDestination = DestinationFolder / ContentFile;
			if (IFileManager::Get().Copy(*FinalDestination, *EachFile) == COPY_OK)
			{
				FilesCopied.Add(FinalDestination);
			}
			else
			{
				UE_LOGF(LogFeaturePack, Warning, "Failed to copy %ls to %ls", *EachFile, *FinalDestination);
			}
		}
	}
}

void FFeaturePackContentSource::InsertAdditionalFeaturePacks()
{
	// copy files in any additional packs (These are for the shared assets)
	EFeaturePackDetailLevel RequiredLevel = EFeaturePackDetailLevel::High;
	for (int32 iExtraPack = 0; iExtraPack < AdditionalFeaturePacks.Num(); iExtraPack++)
	{
		FString DestinationFolder = FPaths::ProjectDir();
		FString FullPath = FPaths::FeaturePackDir() + AdditionalFeaturePacks[iExtraPack].GetFeaturePackNameForLevel(RequiredLevel);
		if (FPlatformFileManager::Get().GetPlatformFile().FileExists(*FullPath))
		{
			FString InsertPath = DestinationFolder + *AdditionalFeaturePacks[iExtraPack].MountName;

			TUniquePtr<FFeaturePackContentSource> NewContentSource = MakeUnique<FFeaturePackContentSource>(FullPath);
			if (NewContentSource->IsDataValid() == true)
			{
				bool bHasSourceFiles = false;
				TArray<FString> FilesCopied;
				FString GameFolder = AdditionalFeaturePacks[iExtraPack].MountName;
				NewContentSource->CopyAdditionalFilesToFolder(DestinationFolder, FilesCopied, bHasSourceFiles, GameFolder);
			}
		}
	}
}

bool FFeaturePackContentSource::InsertAdditionalResources(TArray<FFeaturePackLevelSet> InAdditionalFeaturePacks,EFeaturePackDetailLevel RequiredLevel, const FString& InDestinationFolder,TArray<FString>& InFilesCopied )
{
	// Build a map of feature packs we have (This would probably be better elsewhere and stored in 2 arrays - one listing .upack packs and one non-upack packs)
	IAddContentDialogModule& AddContentDialogModule = FModuleManager::LoadModuleChecked<IAddContentDialogModule>("AddContentDialog");
	TMap<FString, FFeaturePackContentSource*> PackMap;
	for (const TSharedRef<IContentSourceProvider>& ContentSourceProvider : *AddContentDialogModule.GetContentSourceProviderManager()->GetContentSourceProviders())
	{
		const TArray<TSharedRef<IContentSource>> ProviderSources = ContentSourceProvider->GetContentSources();
		for (const TSharedRef<IContentSource>& EachSourceProvider : ProviderSources)
		{
			FFeaturePackContentSource* Source = (FFeaturePackContentSource*) &EachSourceProvider.Get();
			FStringView ID = EachSourceProvider->GetIdent();
			if (!ID.IsEmpty())
			{
				PackMap.Add(FString(ID), Source);
			}
		}
	}

	int32 PacksInserted = 0;
	for (int32 iExtraPack = 0; iExtraPack < InAdditionalFeaturePacks.Num(); iExtraPack++)
	{		
		// TODO - improve error handling here
		FString PackName = InAdditionalFeaturePacks[iExtraPack].GetFeaturePackNameForLevel(RequiredLevel);
		// If we have a 'non upack' pack, insert it else try a upack version
		FFeaturePackContentSource* Insertable = PackMap.FindRef(PackName.Replace(TEXT(".upack"),TEXT("")));
		TArray<FString> FilesCopied;
		if( Insertable != nullptr )
		{
			bool bHasSourceFiles = false;
			Insertable->CopyAdditionalFilesToFolder(InDestinationFolder, FilesCopied, bHasSourceFiles, InAdditionalFeaturePacks[iExtraPack].MountName);
			PacksInserted++;
		}
		else
		{
			FString FullPath = FPaths::FeaturePackDir() + InAdditionalFeaturePacks[iExtraPack].GetFeaturePackNameForLevel(RequiredLevel);

			if (FPlatformFileManager::Get().GetPlatformFile().FileExists(*FullPath))
			{
				FString InsertPath = InDestinationFolder + *InAdditionalFeaturePacks[iExtraPack].MountName;

				TUniquePtr<FFeaturePackContentSource> NewContentSource = MakeUnique<FFeaturePackContentSource>(FullPath);
				if (NewContentSource->IsDataValid() == true)
				{
					bool bHasSourceFiles = false;					
					NewContentSource->CopyAdditionalFilesToFolder(InDestinationFolder, FilesCopied, bHasSourceFiles, InAdditionalFeaturePacks[iExtraPack].MountName);
					PacksInserted++;
				}
				else
				{
					UE_LOGF(LogFeaturePack, Warning, "Unable to install feature pack: '%ls'", *FullPath);
				}
			}
			else
			{
				UE_LOGF(LogFeaturePack, Warning, "Unable to find feature pack: '%ls'", *FullPath);
			}
		}
		// Add to the list of files copied
		InFilesCopied.Append(FilesCopied);
	}
	return PacksInserted == InAdditionalFeaturePacks.Num();
}

bool FFeaturePackContentSource::ParseManifestString(const FString& ManifestString)
{
	// Populate text fields from the manifest.
	TSharedPtr<FJsonObject> ManifestObject;
	TSharedRef<TJsonReader<>> ManifestReader = TJsonReaderFactory<>::Create(ManifestString);
	FJsonSerializer::Deserialize(ManifestReader, ManifestObject);

	if (ManifestReader->GetErrorMessage().IsEmpty() == false)
	{
		RecordAndLogError(FString::Printf(TEXT("Error in Feature pack %s. Failed to parse manifest: %s"), *FeaturePackPath, *ManifestReader->GetErrorMessage()));
		Categories = { EContentSourceCategory::Unknown };
		return false;
	}

	if (ManifestObject->HasTypedField<EJson::String>(TEXT("Version")) == true)
	{
		VersionNumber = ManifestObject->GetStringField(TEXT("Version"));
	}
	if (ManifestObject->HasTypedField<EJson::String>(TEXT("Ident")) == true)
	{
		Identity = ManifestObject->GetStringField(TEXT("Ident"));
	}

	TSharedPtr<FString> ManifestObjectErrorMessage;
	if (TryValidateManifestObject(ManifestObject, ManifestObjectErrorMessage) == false)
	{
		RecordAndLogError(FString::Printf(TEXT("Error in Feature pack %s. Manifest object error: %s"), *FeaturePackPath, **ManifestObjectErrorMessage));
		Categories = { EContentSourceCategory::Unknown };
		return false;
	}

	for (TSharedPtr<FJsonValue> NameValue : ManifestObject->GetArrayField(TEXT("Name")))
	{
		TSharedPtr<FJsonObject> LocalizedNameObject = NameValue->AsObject();
		LocalizedNames.Add(FLocalizedText(
			LocalizedNameObject->GetStringField(TEXT("Language")),
			FText::FromString(LocalizedNameObject->GetStringField(TEXT("Text")))));
	}

	for (TSharedPtr<FJsonValue> DescriptionValue : ManifestObject->GetArrayField(TEXT("Description")))
	{
		TSharedPtr<FJsonObject> LocalizedDescriptionObject = DescriptionValue->AsObject();
		LocalizedDescriptions.Add(FLocalizedText(
			LocalizedDescriptionObject->GetStringField(TEXT("Language")),
			FText::FromString(LocalizedDescriptionObject->GetStringField(TEXT("Text")))));
	}

	// Parse asset types field
	for (TSharedPtr<FJsonValue> AssetTypesValue : ManifestObject->GetArrayField(TEXT("AssetTypes")))
	{
		TSharedPtr<FJsonObject> LocalizedAssetTypesObject = AssetTypesValue->AsObject();
		LocalizedAssetTypesList.Add(FLocalizedText(
			LocalizedAssetTypesObject->GetStringField(TEXT("Language")),
			FText::FromString(LocalizedAssetTypesObject->GetStringField(TEXT("Text")))));
	}

	// Parse search tags field
	if (ManifestObject->HasField(TEXT("SearchTags")) == true)
	{
		for (TSharedPtr<FJsonValue> AssetTypesValue : ManifestObject->GetArrayField(TEXT("SearchTags")))
		{
			TSharedPtr<FJsonObject> LocalizedAssetTypesObject = AssetTypesValue->AsObject();
			LocalizedSearchTags.Add(FLocalizedTextArray(
				LocalizedAssetTypesObject->GetStringField(TEXT("Language")),
				LocalizedAssetTypesObject->GetStringField(TEXT("Text"))));
		}
	}

	// Parse class types field
	ClassTypes = ManifestObject->GetStringField(TEXT("ClassTypes"));

	// Parse initial focus asset if we have one - this is not required
	if (ManifestObject->HasTypedField<EJson::String>(TEXT("FocusAsset")) == true)
	{
		FocusAssetIdent = ManifestObject->GetStringField(TEXT("FocusAsset"));
	}

	// Use the path as the sort key - it will be alphabetical that way
	SortKey = FeaturePackPath;
	ManifestObject->TryGetStringField(TEXT("SortKey"), SortKey);

	TArray<FString> CategoryStrings;
	if (!ManifestObject->TryGetStringArrayField(TEXT("Category"), CategoryStrings))
	{
		FString CategoryString = ManifestObject->GetStringField(TEXT("Category"));
		if (!CategoryString.IsEmpty())
		{
			CategoryStrings.Add(CategoryString);
		}
	}

	UEnum* Enum = FindObjectChecked<UEnum>(nullptr, TEXT("/Script/AddContentDialog.EContentSourceCategory"));
	for (const FString& CategoryString : CategoryStrings)
	{
		int32 EnumValue = Enum->GetValueByName(FName(*CategoryString));
		if (EnumValue != INDEX_NONE)
		{
			Categories.Add((EContentSourceCategory) EnumValue);
		}
	}

	// Thumbnail filename
	IconFilename = ManifestObject->GetStringField(TEXT("Thumbnail"));

	// Screenshots filenames
	ScreenshotFilenameArray = ManifestObject->GetArrayField(TEXT("Screenshots"));
	
	// Parse additional files data
	if (ManifestObject->HasTypedField<EJson::Object>(TEXT("AdditionalFiles")) == true)
	{
		TSharedPtr<FJsonObject> AdditionalFileObject = ManifestObject->GetObjectField(TEXT("AdditionalFiles"));
		if( AdditionalFileObject->HasTypedField<EJson::String>(TEXT("DestinationFilesFolder")) == true )
		{
			AdditionalFilesForPack.DestinationFilesFolder = AdditionalFileObject->GetStringField(TEXT("DestinationFilesFolder"));
			if (AdditionalFileObject->HasTypedField<EJson::Array>(TEXT("AdditionalFilesList")) == true)
			{
				for (TSharedPtr<FJsonValue> FileEntryValue : AdditionalFileObject->GetArrayField(TEXT("AdditionalFilesList")))
				{
					const FString FileSpecString = FileEntryValue->AsString();
					AdditionalFilesForPack.AdditionalFilesList.AddUnique(FileSpecString);
				}
			}
		}
	}

	// Parse additional packs data
	if (ManifestObject->HasTypedField<EJson::Array>(TEXT("AdditionalFeaturePacks")) == true)
	{
		UEnum* DetailEnum = FindObjectChecked<UEnum>(nullptr, TEXT("/Script/AddContentDialog.EFeaturePackDetailLevel"));
		for (TSharedPtr<FJsonValue> AdditionalFeaturePackValue : ManifestObject->GetArrayField(TEXT("AdditionalFeaturePacks")))
		{
			TSharedPtr<FJsonObject> EachAdditionalPack = AdditionalFeaturePackValue->AsObject();
			FString MountName = EachAdditionalPack->GetStringField(TEXT("MountName"));

			TArray<EFeaturePackDetailLevel>	Levels;
			for (TSharedPtr<FJsonValue> DetailValue : EachAdditionalPack->GetArrayField(TEXT("DetailLevels")))
			{
				const FString DetailString = DetailValue->AsString();
				int32 eValue = DetailEnum->GetValueByName(FName(*DetailString));
				EFeaturePackDetailLevel EachLevel = eValue != INDEX_NONE ? (EFeaturePackDetailLevel)eValue : EFeaturePackDetailLevel::Standard;
				Levels.AddUnique(EachLevel);
			}
			AdditionalFeaturePacks.Add(FFeaturePackLevelSet(MountName,Levels));
		}
	}
	
	bPackValid = true;
	return true;
}

bool FFeaturePackContentSource::LoadFeaturePackImageData()
{
	TSharedPtr<TArray<uint8>> IconImageData = MakeShareable(new TArray<uint8>());
 	FString ThumbnailFile = FPaths::Combine(*MountPoint, TEXT("Media"), *IconFilename);
 	if (FFileHelper::LoadFileToArray( *IconImageData.Get(),*ThumbnailFile ) == true)
 	{
 		IconData = MakeShareable(new FImageData(IconFilename, IconImageData));
 	}
 	else
 	{
 		RecordAndLogError(FString::Printf(TEXT("Error in Feature pack %s. Cannot find thumbnail %s."), *FeaturePackPath, *ThumbnailFile));
 	}

	// parse the screenshots field
	for (const TSharedPtr<FJsonValue>& ScreenshotFilename : ScreenshotFilenameArray)
	{
		TSharedPtr<TArray<uint8>> SingleScreenshotData = MakeShareable(new TArray<uint8>);
 		if (FFileHelper::LoadFileToArray(*SingleScreenshotData.Get(), *FPaths::Combine(*MountPoint, TEXT("Media"), *ScreenshotFilename->AsString()) ))
		{
			ScreenshotData.Add(MakeShareable(new FImageData(ScreenshotFilename->AsString(), SingleScreenshotData)));
		}
		else
		{
 			RecordAndLogError(FString::Printf(TEXT("Error in Feature pack %s. Cannot find screenshot %s."), *FeaturePackPath, *ScreenshotFilename->AsString()));
 		}
	}
	return true;
}

bool FFeaturePackContentSource::LoadFeaturePackImageDataFromPackFile(FPakPlatformFile& PakPlatformFile)
{
	bool bAllImagesValid = true;
	TSharedPtr<TArray<uint8>> IconImageData = MakeShareable(new TArray<uint8>());
	FString ThumbnailFile = FPaths::Combine(*MountPoint, TEXT("Media"), *IconFilename);
	if (LoadPakFileToBuffer(PakPlatformFile, ThumbnailFile, *IconImageData) == true)
	{
		IconData = MakeShareable(new FImageData(IconFilename, IconImageData));
	}
	else
	{
		RecordAndLogError(FString::Printf(TEXT("Error in Feature pack %s. Cannot find thumbnail %s."), *FeaturePackPath, *ThumbnailFile));
		bAllImagesValid = false;
	}

	// parse the screenshots field
	for (const TSharedPtr<FJsonValue>& ScreenshotFilename : ScreenshotFilenameArray)
	{
		TSharedPtr<TArray<uint8>> SingleScreenshotData = MakeShareable(new TArray<uint8>);
		if (LoadPakFileToBuffer(PakPlatformFile, FPaths::Combine(*MountPoint, TEXT("Media"), *ScreenshotFilename->AsString()), *SingleScreenshotData))
		{
			ScreenshotData.Add(MakeShareable(new FImageData(ScreenshotFilename->AsString(), SingleScreenshotData)));
		}
		else
		{
			RecordAndLogError(FString::Printf(TEXT("Error in Feature pack %s. Cannot find screenshot %s."), *FeaturePackPath, *ScreenshotFilename->AsString()));
			bAllImagesValid = false;
		}
	}
	return bAllImagesValid;
}

#undef LOCTEXT_NAMESPACE 
