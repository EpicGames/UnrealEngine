// Copyright Epic Games, Inc. All Rights Reserved.

#include "ContentSourceProviders/FeaturePack/FeaturePackContentSourceProvider.h"

#include "Delegates/Delegate.h"
#include "DirectoryWatcherModule.h"
#include "FeaturePackContentSource.h"
#include "GenericPlatform/GenericPlatformFile.h"
#include "HAL/FileManager.h"
#include "HAL/Platform.h"
#include "HAL/PlatformCrt.h"
#include "HAL/PlatformFileManager.h"
#include "IContentSource.h"
#include "Misc/Paths.h"
#include "Modules/ModuleManager.h"
#include "Templates/UniquePtr.h"


class FFillArrayDirectoryVisitor : public IPlatformFile::FDirectoryVisitor
{
public:
	virtual bool Visit(const TCHAR* FilenameOrDirectory, bool bIsDirectory) override
	{
		if (bIsDirectory)
		{
			Directories.Add(FilenameOrDirectory);
		}
		else
		{
			Files.Add(FilenameOrDirectory);
		}
		return true;
	}

	TArray<FString> Directories;
	TArray<FString> Files;
};

FFeaturePackContentSourceProvider::FFeaturePackContentSourceProvider()
{
	FeaturePackPath = FPaths::FeaturePackDir();
	EnterpriseFeaturePackPath = FPaths::EnterpriseFeaturePackDir();
	TemplatePath = FPaths::RootDir() + TEXT("Templates/");
	EnterpriseTemplatePath = FPaths::EnterpriseDir() + TEXT("Templates/");
	StartUpDirectoryWatcher();
}

const TArray<TSharedRef<IContentSource>>& FFeaturePackContentSourceProvider::GetContentSources() const
{
	if (!bAreContentSourcesUpToDate)
	{
		// Using mutable to avoid removing the const interface since this is just a caching mechanism.
		FFeaturePackContentSourceProvider* MutableThis = const_cast<FFeaturePackContentSourceProvider*>(this);
		MutableThis->RefreshFeaturePacks();
	}

	return ContentSources;
}

void FFeaturePackContentSourceProvider::SetContentSourcesChanged(FOnContentSourcesChanged OnContentSourcesChangedIn)
{
	OnContentSourcesChanged = OnContentSourcesChangedIn;
}

void FFeaturePackContentSourceProvider::StartUpDirectoryWatcher()
{
	FDirectoryWatcherModule& DirectoryWatcherModule = FModuleManager::LoadModuleChecked<FDirectoryWatcherModule>(TEXT("DirectoryWatcher"));
	IDirectoryWatcher* DirectoryWatcher = DirectoryWatcherModule.Get();
	if (DirectoryWatcher)
	{
		// If the path doesn't exist on disk, make it so the watcher will work.
		IFileManager::Get().MakeDirectory(*FeaturePackPath);
		DirectoryChangedDelegate = IDirectoryWatcher::FDirectoryChanged::CreateRaw(this, &FFeaturePackContentSourceProvider::OnFeaturePackDirectoryChanged);
		DirectoryWatcher->RegisterDirectoryChangedCallback_Handle(FeaturePackPath, DirectoryChangedDelegate, DirectoryChangedDelegateHandle);

		if (IFileManager::Get().DirectoryExists(*FPaths::EnterpriseDir()))
		{
			// If the path doesn't exist on disk, make it so the watcher will work.
			IFileManager::Get().MakeDirectory(*EnterpriseFeaturePackPath);
			DirectoryWatcher->RegisterDirectoryChangedCallback_Handle(EnterpriseFeaturePackPath, DirectoryChangedDelegate, DirectoryChangedDelegateHandle);
		}
	}
}

void FFeaturePackContentSourceProvider::ShutDownDirectoryWatcher()
{
	FDirectoryWatcherModule& DirectoryWatcherModule = FModuleManager::LoadModuleChecked<FDirectoryWatcherModule>( TEXT( "DirectoryWatcher" ) );
	IDirectoryWatcher* DirectoryWatcher = DirectoryWatcherModule.Get();
	if ( DirectoryWatcher )
	{
		DirectoryWatcher->UnregisterDirectoryChangedCallback_Handle(FeaturePackPath, DirectoryChangedDelegateHandle);
		DirectoryWatcher->UnregisterDirectoryChangedCallback_Handle(EnterpriseFeaturePackPath, DirectoryChangedDelegateHandle);
	}
}

void FFeaturePackContentSourceProvider::OnFeaturePackDirectoryChanged( const TArray<FFileChangeData>& FileChanges )
{
	bAreContentSourcesUpToDate = false;
}

/** Sorting function sort keys. */
struct FFeaturePackCompareSortKey
{
	FORCEINLINE bool operator()( TSharedRef<IContentSource> const& A, TSharedRef<IContentSource> const& B) const { return A->GetSortKey() < B->GetSortKey(); }
};

void FFeaturePackContentSourceProvider::RefreshFeaturePacks()
{
	TRACE_CPUPROFILER_EVENT_SCOPE(FFeaturePackContentSourceProvider::RefreshFeaturePacks);
	
	ContentSources.Empty();
	// TODO improve this and seperate handling of .upack and manifest (loose) parsing
	// first the .upack files
	IPlatformFile &PlatformFile = FPlatformFileManager::Get().GetPlatformFile();
	FFillArrayDirectoryVisitor DirectoryVisitor;
	PlatformFile.IterateDirectory(*FeaturePackPath, DirectoryVisitor);
	PlatformFile.IterateDirectory(*EnterpriseFeaturePackPath, DirectoryVisitor);
	for ( auto FeaturePackFile : DirectoryVisitor.Files )
	{
		if( FeaturePackFile.EndsWith(TEXT(".upack")) == true)
		{
			TUniquePtr<FFeaturePackContentSource> NewContentSource = MakeUnique<FFeaturePackContentSource>(FeaturePackFile);
			if (NewContentSource->IsDataValid())
			{
				ContentSources.Add(MakeShareable(NewContentSource.Release()));
			}
		}
	}
	
	// Now the 'loose' feature packs
	FFillArrayDirectoryVisitor TemplateDirectoryVisitor;
	PlatformFile.IterateDirectoryRecursively(*TemplatePath, TemplateDirectoryVisitor);
	PlatformFile.IterateDirectoryRecursively(*EnterpriseTemplatePath, TemplateDirectoryVisitor);
	for (auto TemplatePackFile: TemplateDirectoryVisitor.Files)
	{
		FString ThisPackRoot = FPaths::GetPath(TemplatePackFile);
		if (ThisPackRoot.EndsWith(TEXT("FeaturePack")) == true)
		{			
			if(TemplatePackFile.EndsWith(TEXT("manifest.json")) == true)
			{
				TUniquePtr<FFeaturePackContentSource> NewContentSource = MakeUnique<FFeaturePackContentSource>(TemplatePackFile);
				if (NewContentSource->IsDataValid())
				{
					ContentSources.Add(MakeShareable(NewContentSource.Release()));
				}
			}
		}
	}

	ContentSources.Sort(FFeaturePackCompareSortKey());
	bAreContentSourcesUpToDate = true;

	// Assume for now that this delegate doesn't support being called from any thread.
	check(IsInGameThread());
	OnContentSourcesChanged.ExecuteIfBound();
}

FFeaturePackContentSourceProvider::~FFeaturePackContentSourceProvider()
{
	ShutDownDirectoryWatcher();
}
