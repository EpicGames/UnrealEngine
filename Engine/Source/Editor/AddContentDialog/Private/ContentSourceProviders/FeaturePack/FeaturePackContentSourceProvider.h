// Copyright Epic Games, Inc. All Rights Reserved.

#pragma once

#include "Containers/Array.h"
#include "Containers/UnrealString.h"
#include "Delegates/IDelegateInstance.h"
#include "IContentSourceProvider.h"
#include "IDirectoryWatcher.h"
#include "Templates/SharedPointer.h"

class IContentSource;

/** A content source provider for available content upacks. */
class FFeaturePackContentSourceProvider : public IContentSourceProvider
{
public:
	FFeaturePackContentSourceProvider();
	virtual const TArray<TSharedRef<IContentSource>>& GetContentSources() const override;
	virtual void SetContentSourcesChanged(FOnContentSourcesChanged OnContentSourcesChangedIn) override;

	virtual ~FFeaturePackContentSourceProvider();

private:
	/** Starts the directory watcher for the feature pack directory. */
	void StartUpDirectoryWatcher();

	/** Shuts down the directory watcher for the feature pack directory. */
	void ShutDownDirectoryWatcher();

	/** Delegate to handle whenever the feature pack directory changes on disk. */
	void OnFeaturePackDirectoryChanged( const TArray<FFileChangeData>& FileChanges );

	/** Rebuilds the feature pack array and calls the change notification delegate. */
	void RefreshFeaturePacks();

	/** The path on disk to the directory containing the feature packs. */
	FString FeaturePackPath;

	/** The path on disk to the directory containing the enterprise feature packs. */
	FString EnterpriseFeaturePackPath;

	/** The path on disk to the directory containing the templates folder. */
	FString TemplatePath;

	/** The path on disk to the directory containing the enterprise templates folder. */
	FString EnterpriseTemplatePath;

	/** The delegate which gets called when the feature pack directory changes. This reference
		is kept so that it can be unregistered correctly. */
	IDirectoryWatcher::FDirectoryChanged DirectoryChangedDelegate;

	/** A delegate which gets called whenever the array of content sources changes. */
	FOnContentSourcesChanged OnContentSourcesChanged;

	/** An array of the available content sources. */
	TArray<TSharedRef<IContentSource>> ContentSources;

	/** An array of the available Enterprise content sources. */
	TArray<TSharedRef<IContentSource>> EnterpiseContentSources;

	/* Are the cached content sources up to date*/
	mutable bool bAreContentSourcesUpToDate = false;

	FDelegateHandle DirectoryChangedDelegateHandle;
};
