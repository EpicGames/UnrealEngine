// Copyright Epic Games, Inc. All Rights Reserved.

#pragma once

#include "CoreMinimal.h"
#include "UObject/ObjectMacros.h"
#include "IContentSource.generated.h"

/** Defines categories for content sources. */
UENUM()
enum class EContentSourceCategory:uint8
{
	BlueprintFeature,
	CodeFeature,
	EnterpriseFeature,
	Content,
	EnterpriseContent,
	SharedPack,
	Unknown
};


/** Represents raw binary image data in png format. */
class FImageData
{
public:
	/** Creates a new FImageData
		@param InName - The name which identifies the image represented by this object 
		@param InData - The raw binary image data in png format */
	FImageData(FString InName, TSharedPtr <TArray<uint8>> InData)
	{
		Name = InName;
		Data = InData;
	}

	/** Gets the name which identifies the image represented by this object. */
	const FString GetName() const
	{
		return Name;
	}

	/** Gets the raw binary image data in PNG format. */
	const TSharedPtr<TArray<uint8>>& GetData() const
	{
		return Data;
	}

private:
	FString Name;
	TSharedPtr<TArray<uint8>> Data;
};

/** Represents a pieces of localized text. */
class FLocalizedText
{
public:
	FLocalizedText()
	{
	}

	/** Creates a new FLocalizedText
		@param InTwoLetterLanguage - The iso 2-letter language specifier.
		@param InText - The text in the language specified */
	FLocalizedText(FString InTwoLetterLanguage, FText InText)
	{
		TwoLetterLanguage = InTwoLetterLanguage;
		Text = InText;
	}

	/** Gets the iso 2-letter language specifier for this text. */
	const FString& GetTwoLetterLanguage() const
	{
		return TwoLetterLanguage;
	}

	/** Gets the text in the language specified. */
	FText GetText() const
	{
		return Text;
	}

private:
	FString TwoLetterLanguage;
	FText Text;
};

/** Defines a source of content to be used with the FAddContentDialog */
class IContentSource
{
public:
	virtual ~IContentSource() = default;

	/** Gets the name of the content source as an array of localized strings. */
	virtual const TArray<FLocalizedText>& GetLocalizedNames() const = 0;

	/** Gets the description of the content source as an array or localized strings. */
	virtual const TArray<FLocalizedText>& GetLocalizedDescriptions() const = 0;

	/** Gets the category for the content source. */
	virtual const TArray<EContentSourceCategory>& GetCategories() const = 0;

	/** Gets the image data for the icon which should represent the content source in the UI. */
	virtual TSharedPtr<FImageData> GetIconData() const = 0;

	/** Gets an array or image data for screenshots for the content source. */
	virtual const TArray<TSharedPtr<FImageData>>& GetScreenshotData() const = 0;

	/** Gets the asset types used in this pack. */
	virtual const TArray<FLocalizedText>& GetLocalizedAssetTypes() const = 0;

	/** Gets the class types used in this pack. */
	virtual const FString& GetClassTypesUsed() const = 0;

	/** Gets the category for the content source. */
	virtual const FString& GetSortKey() const = 0;
	/*
	 * Installs the content in the content source to the specific path. 
	 * @returns true if install succeeded
	 */
	virtual bool InstallToProject(FString InstallPath) = 0;

	/** Is the data in this content valid. */
	virtual bool IsDataValid() const = 0;

	/** Gets the identity of the content. */
	virtual const FString& GetIdent() const = 0;
};
