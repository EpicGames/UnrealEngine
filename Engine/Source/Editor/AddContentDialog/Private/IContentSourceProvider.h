// Copyright Epic Games, Inc. All Rights Reserved.

#pragma once

#include "CoreMinimal.h"
#include "IContentSource.h"

/** Defines methods for an object which provides IContentSource objects for use with the SAddContentDialog. */
class IContentSourceProvider
{
public:
	DECLARE_DELEGATE(FOnContentSourcesChanged);

public:
	/** Gets the available content sources. */
	virtual const TArray<TSharedRef<IContentSource>>& GetContentSources() const = 0;

	/** Sets the delegate which will be executed when the avaialble content sources change */
	virtual void SetContentSourcesChanged(FOnContentSourcesChanged OnContentSourcesChangedIn) = 0;

	virtual ~IContentSourceProvider() = default;
};
