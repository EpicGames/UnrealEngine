// Copyright Epic Games, Inc. All Rights Reserved.

#include "SAddContentDialog.h"

#include "Internationalization/Internationalization.h"
#include "Layout/Margin.h"
#include "Math/Vector2D.h"
#include "Misc/Attribute.h"
#include "SAddContentWidget.h"
#include "Styling/AppStyle.h"
#include "Widgets/Layout/SBorder.h"

#define LOCTEXT_NAMESPACE "AddContentDialog"

void SAddContentDialog::Construct(const FArguments& InArgs)
{

	SWindow::Construct(SWindow::FArguments()
		.Title(LOCTEXT("AddContentDialogTitle", "Add Content to the Project"))
		.SizingRule(ESizingRule::UserSized)
		.ClientSize(FVector2D(900, 500))
		.SupportsMinimize(false)
		.SupportsMaximize(false)
		[
			SNew(SBorder)
			.BorderImage(FAppStyle::GetBrush("Brushes.Panel"))
			.Padding(FMargin(10,0))
			[
				SNew(SAddContentWidget)
			]
		]);
}

SAddContentDialog::~SAddContentDialog()
{
}

#undef LOCTEXT_NAMESPACE
