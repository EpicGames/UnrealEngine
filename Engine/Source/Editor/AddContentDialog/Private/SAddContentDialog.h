// Copyright Epic Games, Inc. All Rights Reserved.

#pragma once

#include "Widgets/DeclarativeSyntaxSupport.h"
#include "Widgets/SWindow.h"

/** A window which allows the user to select additional content to add to the currently loaded project. */
class SAddContentDialog : public SWindow
{
public:
	SLATE_BEGIN_ARGS(SWindow)
	{}

	SLATE_END_ARGS()

	~SAddContentDialog();

	void Construct(const FArguments& InArgs);

private:
};
