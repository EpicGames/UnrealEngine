// Copyright Epic Games, Inc. All Rights Reserved.

#include "SAddContentWidget.h"

#include "Containers/StringFwd.h"
#include "Containers/StringView.h"
#include "Fonts/SlateFontInfo.h"
#include "Framework/Application/SlateApplication.h"
#include "Framework/Views/ITypedTableView.h"
#include "HAL/Platform.h"
#include "IContentSource.h"
#include "Internationalization/BreakIterator.h"
#include "Internationalization/Internationalization.h"
#include "Internationalization/Text.h"
#include "Layout/BasicLayoutWidgetSlot.h"
#include "Layout/Children.h"
#include "Layout/Margin.h"
#include "Layout/Visibility.h"
#include "Math/Vector2D.h"
#include "Misc/Attribute.h"
#include "SPrimaryButton.h"
#include "SlotBase.h"
#include "Styling/AppStyle.h"
#include "Styling/ISlateStyle.h"
#include "Styling/SlateColor.h"
#include "Styling/StyleColors.h"
#include "Styling/StyleDefaults.h"
#include "SWidgetCarouselWithNavigation.h"
#include "Templates/TypeHash.h"
#include "Types/SlateStructs.h"
#include "UObject/NameTypes.h"
#include "ViewModels/AddContentWidgetViewModel.h"
#include "ViewModels/CategoryViewModel.h"
#include "ViewModels/ContentSourceViewModel.h"
#include "WidgetCarouselStyle.h"
#include "Widgets/Images/SImage.h"
#include "Widgets/Input/SButton.h"
#include "Widgets/Input/SSearchBox.h"
#include "Widgets/Input/SSegmentedControl.h"
#include "Widgets/Layout/SBorder.h"
#include "Widgets/Layout/SBox.h"
#include "Widgets/Layout/SScrollBox.h"
#include "Widgets/Layout/SSeparator.h"
#include "Widgets/SBoxPanel.h"
#include "Widgets/SOverlay.h"
#include "Widgets/SWindow.h"
#include "Widgets/Text/STextBlock.h"
#include "Widgets/Views/STableRow.h"

class FString;
class ITableRow;
class SWidget;
struct FSlateBrush;

#define LOCTEXT_NAMESPACE "AddContentDialog"

/** 
 *
 */
class SGenericThumbnailTile : public SCompoundWidget
{
	SLATE_BEGIN_ARGS(SGenericThumbnailTile)
		: _ThumbnailSize(102, 102)
		, _ThumbnailPadding(FMargin(5.0f, 0))
		, _IsSelected(false)
	{}
		SLATE_ATTRIBUTE(const FSlateBrush*, Image)
		SLATE_ATTRIBUTE(FText, DisplayName)
		/** The size of the thumbnail */
		SLATE_ARGUMENT(FVector2D, ThumbnailSize)
		/** The size of the image in the thumbnail. Used to make the thumbnail not fill the entire thumbnail area. If this is not set the Image brushes desired size is used */
		SLATE_ARGUMENT(TOptional<FVector2D>, ImageSize)
		SLATE_ARGUMENT(FMargin, ThumbnailPadding)
		SLATE_ATTRIBUTE(bool, IsSelected)
	SLATE_END_ARGS()

	void Construct(const FArguments& InArgs)
	{
		Image = InArgs._Image;
		Text = InArgs._DisplayName;
		IsSelected = InArgs._IsSelected;

		ChildSlot
		[
			SNew(SBorder)
			.Padding(FMargin(0.0f, 0.0f, 5.0f, 5.0f))
			.BorderImage(FAppStyle::Get().GetBrush("ProjectBrowser.ProjectTile.DropShadow"))
			[
				SNew(SOverlay)
				+ SOverlay::Slot()
				[
					SNew(SVerticalBox)
					// Thumbnail
					+ SVerticalBox::Slot()
					.AutoHeight()
					.HAlign(HAlign_Center)
					.VAlign(VAlign_Center)
					[
						SNew(SBox)
						.WidthOverride(InArgs._ThumbnailSize.X)
						.HeightOverride(InArgs._ThumbnailSize.Y)
						[
							SNew(SBorder)
							.Padding(0)
							.BorderImage(FAppStyle::Get().GetBrush("ProjectBrowser.ProjectTile.ThumbnailAreaBackground"))
							.HAlign(HAlign_Center)
							.VAlign(VAlign_Center)
							[
								SNew(SImage)
								.Image(Image)
								.DesiredSizeOverride(InArgs._ImageSize)
							]
						]
					]
					// Name
					+ SVerticalBox::Slot()
					[
						SNew(SBorder)
						.Padding(InArgs._ThumbnailPadding)
						.VAlign(VAlign_Top)
						.Padding(FMargin(3.0f, 3.0f))
						.BorderImage(FAppStyle::Get().GetBrush("ProjectBrowser.ProjectTile.NameAreaBackground"))
						[
							SNew(STextBlock)
							.Font(FAppStyle::Get().GetFontStyle("ProjectBrowser.ProjectTile.Font"))
							.AutoWrapText(true)
							.LineBreakPolicy(FBreakIterator::CreateWordBreakIterator())
							.Text(InArgs._DisplayName)
							.ColorAndOpacity(FAppStyle::Get().GetSlateColor("Colors.Foreground"))
						]
					]
				]
				+ SOverlay::Slot()
				[
					SNew(SImage)
					.Visibility(EVisibility::HitTestInvisible)
					.Image_Lambda
					(
						[this]()
						{
							const bool bSelected = IsSelected.Get();
							const bool bHovered = IsHovered();

							if (bSelected && bHovered)
							{
								static const FName SelectedHover("ProjectBrowser.ProjectTile.SelectedHoverBorder");
								return FAppStyle::Get().GetBrush(SelectedHover);
							}
							else if (bSelected)
							{
								static const FName Selected("ProjectBrowser.ProjectTile.SelectedBorder");
								return FAppStyle::Get().GetBrush(Selected);
							}
							else if (bHovered)
							{
								static const FName Hovered("ProjectBrowser.ProjectTile.HoverBorder");
								return FAppStyle::Get().GetBrush(Hovered);
							}

							return FStyleDefaults::GetNoBrush();
						}
					)
				]
			]
		];
	}

private:
	TAttribute<const FSlateBrush*> Image;
	TAttribute<FText> Text;
	TAttribute<bool> IsSelected;
};


void SAddContentWidget::Construct(const FArguments& InArgs)
{
	ViewModel = FAddContentWidgetViewModel::CreateShared();
	ViewModel->SetOnCategoriesChanged(FAddContentWidgetViewModel::FOnCategoriesChanged::CreateSP(
		this, &SAddContentWidget::CategoriesChanged));
	ViewModel->SetOnContentSourcesChanged(FAddContentWidgetViewModel::FOnContentSourcesChanged::CreateSP(
		this, &SAddContentWidget::ContentSourcesChanged));
	ViewModel->SetOnSelectedContentSourceChanged(FAddContentWidgetViewModel::FOnSelectedContentSourceChanged::CreateSP(
		this, &SAddContentWidget::SelectedContentSourceChanged));

	ChildSlot
	[
		SNew(SVerticalBox)
		+ SVerticalBox::Slot()
		[
			SNew(SHorizontalBox)
			// Content Source Tiles
			+ SHorizontalBox::Slot()
			[
				SNew(SVerticalBox)
				+ SVerticalBox::Slot()
				.AutoHeight()
				.Padding(FMargin(0, 10, 0, 16))
				.HAlign(HAlign_Center)
				[
					SAssignNew(CategoryTabsContainer, SBox)
					[
						CreateCategoryTabs()
					]
				]
				// Content Source Filter
				+ SVerticalBox::Slot()
				.AutoHeight()
				.Padding(FMargin(0, 0, 0, 8))
				[
					SAssignNew(SearchBoxPtr, SSearchBox)
					.OnTextChanged(this, &SAddContentWidget::SearchTextChanged)
				]

				// Content Source Tile View
				+ SVerticalBox::Slot()
				[
					CreateContentSourceTileView()
				]
			]

			// Splitter
			+ SHorizontalBox::Slot()
			.Padding(FMargin(18, 0, 0, 0))
			.AutoWidth()
			[
				SNew(SSeparator)
				.Orientation(Orient_Vertical)
				.Thickness(2.0f)
			]

			// Content Source Details
			+ SHorizontalBox::Slot()
			[
				SAssignNew(ContentSourceDetailContainer, SBox)
				[
					CreateContentSourceDetail(ViewModel->GetSelectedContentSource())
				]
			]
		]
		+ SVerticalBox::Slot()
		.AutoHeight()
		[
			SNew(SSeparator)
			.Thickness(2.0f)
		]
		+ SVerticalBox::Slot()
		.AutoHeight()
		.Padding(0, 16, 0, 16)
		[
			SNew(SHorizontalBox)
			+ SHorizontalBox::Slot()
			.VAlign(VAlign_Center)
			.HAlign(HAlign_Right)
			[
				SNew(SPrimaryButton)
				.OnClicked(this, &SAddContentWidget::AddButtonClicked)
				.Text(LOCTEXT("AddToProjectButton", "Add to Project"))
			]
			+ SHorizontalBox::Slot()
			.AutoWidth()
			.VAlign(VAlign_Center)
			.Padding(8.0f, 0.0f, 0.0f, 0.0f)
			[
				SNew(SButton)
				.TextStyle(FAppStyle::Get(), "DialogButtonText")
				.OnClicked(this, &SAddContentWidget::CancelButtonClicked)
				.Text(LOCTEXT("Cancel", "Cancel"))
			]
		]
	];
}

const TArray<TSharedPtr<IContentSource>>* SAddContentWidget::GetContentSourcesToAdd()
{
	return &ContentSourcesToAdd;
}

TSharedRef<SWidget> SAddContentWidget::CreateCategoryTabs()
{
	TSharedRef<SSegmentedControl<FCategoryViewModel>> CategoriesWidget =
		SNew(SSegmentedControl<FCategoryViewModel>)
		.OnValueChanged(this, &SAddContentWidget::OnSelectedCategoryChanged)
		.Value(this, &SAddContentWidget::GetSelectedCategory)
		.TextStyle(FAppStyle::Get(), "DialogButtonText")
		.MaxSegmentsPerLine(3);

	const bool bRebuildChildren = false;
	int32 Index = 0;
	for (const FCategoryViewModel& Category : ViewModel->GetCategories())
	{
		CategoriesWidget->AddSlot(Category, bRebuildChildren)
			.Text(Category.GetText());
	}

	CategoriesWidget->RebuildChildren();

	return CategoriesWidget;
}

TSharedRef<SWidget> SAddContentWidget::CreateContentSourceTileView()
{
	SAssignNew(ContentSourceTileView, STileView<TSharedPtr<FContentSourceViewModel>>)
	.ListItemsSource(ViewModel->GetContentSources())
	.OnGenerateTile(this, &SAddContentWidget::CreateContentSourceIconTile)
	.OnSelectionChanged(this, &SAddContentWidget::ContentSourceTileViewSelectionChanged)
	.ClearSelectionOnClick(false)
	.ItemAlignment(EListItemAlignment::LeftAligned)
	.ItemWidth(102)
	.ItemHeight(153)
	.SelectionMode(ESelectionMode::Single);

	ContentSourceTileView->SetSelection(ViewModel->GetSelectedContentSource(), ESelectInfo::Direct);
	return ContentSourceTileView.ToSharedRef();
}

TSharedRef<ITableRow> SAddContentWidget::CreateContentSourceIconTile(TSharedPtr<FContentSourceViewModel> ContentSource, const TSharedRef<STableViewBase>& OwnerTable)
{
	TWeakPtr<FContentSourceViewModel> ContentSourceWeakPtr = ContentSource;

	return 
		SNew(STableRow< TSharedPtr<FString> >, OwnerTable)
		.Style(FAppStyle::Get(), "ProjectBrowser.TableRow")
		.Padding(2.0f)
		[
			SNew(SGenericThumbnailTile)
			.Image(ContentSource->GetIconBrush().Get())
			.DisplayName(ContentSource->GetName())
			.IsSelected_Lambda([ContentSourceWeakPtr, this] { return ViewModel->GetSelectedContentSource() == ContentSourceWeakPtr; })
		];
}

TSharedRef<SWidget> SAddContentWidget::CreateContentSourceDetail(TSharedPtr<FContentSourceViewModel> ContentSource)
{
	TSharedRef<SVerticalBox> VerticalBox = SNew(SVerticalBox);
	if (ContentSource.IsValid())
	{
		VerticalBox->AddSlot()
		[
			SNew(SScrollBox)
			+ SScrollBox::Slot()
			.Padding(FMargin(0, 0, 0, 5))
			.HAlign(EHorizontalAlignment::HAlign_Left)
			[
				CreateScreenshotCarousel(ContentSource)
			]

			+SScrollBox::Slot()
			.Padding(FMargin(10, 0, 0, 5))
			[
				SNew(STextBlock)
				.TextStyle(FAppStyle::Get(), "DialogButtonText")
				.Font(FAppStyle::Get().GetFontStyle("HeadingExtraSmall"))
				.ColorAndOpacity(FStyleColors::ForegroundHover)
				.Text(ContentSource->GetName())
				.AutoWrapText(true)
			]

			+ SScrollBox::Slot()
			.Padding(FMargin(10, 0, 0, 5))
			[
				SNew(STextBlock)
				.Text(ContentSource->GetDescription())
				.AutoWrapText(true)
			]
		
			+ SScrollBox::Slot()
			.Padding(FMargin(10, 0, 0, 0))
			[
				SNew(STextBlock)						
				.Visibility(ContentSource->GetAssetTypes().IsEmpty() == false ? EVisibility::Visible : EVisibility::Collapsed)
				.TextStyle(FAppStyle::Get(), "DialogButtonText")
				.Text(LOCTEXT("FeaturePackAssetReferences", "Asset Types Used"))
			]
			+ SScrollBox::Slot()
			.Padding(FMargin(10, 0, 0, 5))
			[
				SNew(STextBlock)
				.Text(ContentSource->GetAssetTypes())
				.Visibility(ContentSource->GetAssetTypes().IsEmpty() == false ? EVisibility::Visible : EVisibility::Collapsed)				
				.AutoWrapText(true)
			]
		
			+ SScrollBox::Slot()
			.Padding(FMargin(10, 0, 0, 0))
			[
				SNew(STextBlock)
				.TextStyle(FAppStyle::Get(), "DialogButtonText")
				.Visibility(ContentSource->GetClassTypes().IsEmpty() == false ? EVisibility::Visible : EVisibility::Collapsed)
				.Text(LOCTEXT("FeaturePackClassReferences", "Class Types Used"))
			] 
			+ SScrollBox::Slot()
			.Padding(FMargin(10, 0, 0, 5))
			[
				SNew(STextBlock)
				.Text(FText::FromStringView(ContentSource->GetClassTypes()))
				.Visibility(ContentSource->GetClassTypes().IsEmpty() == false ? EVisibility::Visible : EVisibility::Collapsed)
				.AutoWrapText(true)
			]
		];
	}
	return VerticalBox;
}

TSharedRef<SWidget> SAddContentWidget::CreateScreenshotCarousel(TSharedPtr<FContentSourceViewModel> ContentSource)
{
	return SNew(SWidgetCarouselWithNavigation<TSharedPtr<FSlateBrush>>)
		.NavigationBarStyle(FWidgetCarouselModuleStyle::Get(), "CarouselNavigationBar")
		.NavigationButtonStyle(FWidgetCarouselModuleStyle::Get(), "CarouselNavigationButton")
		.OnGenerateWidget(this, &SAddContentWidget::CreateScreenshotWidget)
		.WidgetItemsSource(&ContentSource->GetScreenshotBrushes());
}

void SAddContentWidget::SearchTextChanged(const FText& SearchText)
{
	ViewModel->SetSearchText(SearchText);
	SearchBoxPtr->SetError(ViewModel->GetSearchErrorText());
}

void SAddContentWidget::ContentSourceTileViewSelectionChanged( TSharedPtr<FContentSourceViewModel> SelectedContentSource, ESelectInfo::Type SelectInfo )
{
	ViewModel->SetSelectedContentSource( SelectedContentSource );
}

FReply SAddContentWidget::AddButtonClicked()
{
	if (ViewModel->GetSelectedContentSource().IsValid())
	{
		ViewModel->GetSelectedContentSource()->GetContentSource()->InstallToProject( "/Game" );
	}
	return FReply::Handled();
}

FReply SAddContentWidget::CancelButtonClicked()
{
	TSharedPtr<SWindow> MyWindow = FSlateApplication::Get().FindWidgetWindow(AsShared());

	MyWindow->RequestDestroyWindow();

	return FReply::Handled();
}

void SAddContentWidget::OnSelectedCategoryChanged(FCategoryViewModel SelectedCategory)
{
	ViewModel->SetSelectedCategory(SelectedCategory);
}

FCategoryViewModel SAddContentWidget::GetSelectedCategory() const
{
	return ViewModel->GetSelectedCategory();
}

TSharedRef<SWidget> SAddContentWidget::CreateScreenshotWidget(TSharedPtr<FSlateBrush> ScreenshotBrush)
{
	return SNew(SImage)
	.Image(ScreenshotBrush.Get());
}

void SAddContentWidget::CategoriesChanged()
{
	CategoryTabsContainer->SetContent(CreateCategoryTabs());
}

void SAddContentWidget::ContentSourcesChanged()
{
	ContentSourceTileView->RequestListRefresh();
}

void SAddContentWidget::SelectedContentSourceChanged()
{
	ContentSourceTileView->SetSelection(ViewModel->GetSelectedContentSource(), ESelectInfo::Direct);
	ContentSourceDetailContainer->SetContent(CreateContentSourceDetail(ViewModel->GetSelectedContentSource()));
}

SAddContentWidget::~SAddContentWidget()
{
}

#undef LOCTEXT_NAMESPACE
