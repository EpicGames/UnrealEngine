// Copyright Epic Games, Inc. All Rights Reserved.

#pragma once

#include "Containers/Array.h"
#include "Containers/BitArray.h"
#include "Containers/Set.h"
#include "Containers/SparseArray.h"
#include "Delegates/Delegate.h"
#include "HAL/PlatformCrt.h"
#include "Input/Reply.h"
#include "Misc/Optional.h"
#include "Templates/SharedPointer.h"
#include "Templates/UnrealTemplate.h"
#include "Types/SlateConstants.h"
#include "Types/SlateEnums.h"
#include "ViewModels/CategoryViewModel.h"
#include "Widgets/DeclarativeSyntaxSupport.h"
#include "Widgets/SCompoundWidget.h"
#include "Widgets/Views/STableViewBase.h"
#include "Widgets/Views/STileView.h"

class FAddContentWidgetViewModel;
class FContentSourceViewModel;
class FText;
class IContentSource;
class ITableRow;
class SBox;
class SWidget;
struct FSlateBrush;

/** A widget which allows the user to select multiple options from content which is available to be added to the project. */
class SAddContentWidget : public SCompoundWidget
{
public:
	DECLARE_DELEGATE(FOnAddListChanged);

	SLATE_BEGIN_ARGS(SAddContentWidget)
	{}

	SLATE_END_ARGS()

	~SAddContentWidget();

	void Construct(const FArguments& InArgs);

	/** Gets the content sources which have been selected by the user for addition to the project */
	const TArray<TSharedPtr<IContentSource>>* GetContentSourcesToAdd();

private:
	/** Creates a strip of tabs which display and allow selecting categories. */
	TSharedRef<SWidget> CreateCategoryTabs();

	/** Creates a tile view which displays the content sources in the selected category. */
	TSharedRef<SWidget> CreateContentSourceTileView();
	
	/** Creates the widget which represents each content source in the content source tile view. */
	TSharedRef<ITableRow> CreateContentSourceIconTile(TSharedPtr<FContentSourceViewModel> ContentSource, const TSharedRef<STableViewBase>& OwnerTable);

	/** Creates a widget representing detailed information about a single content source. */
	TSharedRef<SWidget> CreateContentSourceDetail(TSharedPtr<FContentSourceViewModel> ContentSource);

	/** Creates a widget carousel for displaying the set of screenshots for a specific content source. */
	TSharedRef<SWidget> CreateScreenshotCarousel(TSharedPtr<FContentSourceViewModel> ContentSource);

	/** Creates the widget the displays a screenshot in the screenshot carousel. */
	TSharedRef<SWidget> CreateScreenshotWidget(TSharedPtr<FSlateBrush> ScreenshotBrush);

	/** Handles the text in the search box changing. */
	void SearchTextChanged(const FText& SearchText);

	/** Handles the selection in the content source tile view changing */
	void ContentSourceTileViewSelectionChanged(TSharedPtr<FContentSourceViewModel> SelectedContentSource, ESelectInfo::Type SelectInfo);

	/** Handles the add content to project button being clicked. */
	FReply AddButtonClicked();

	/** Handles the close button being clicked. */
	FReply CancelButtonClicked();

	/** Handles selected category changing in the UI */
	void OnSelectedCategoryChanged(FCategoryViewModel SelectedCategory);

	/** Gets the curerently selected category */
	FCategoryViewModel GetSelectedCategory() const;

	/** Handles the available categories changing on the view model. */
	void CategoriesChanged();

	/** Handles the available content sources changing on the view model. */
	void ContentSourcesChanged();

	/** Handles the selected content source changing on the view model. */
	void SelectedContentSourceChanged();

private:
	/** The view model which represents the current data of the UI. */
	TSharedPtr<FAddContentWidgetViewModel> ViewModel;

	/** The tile view which displays the content sources in the currently selected category. */
	TSharedPtr<STileView<TSharedPtr<FContentSourceViewModel>>> ContentSourceTileView;

	/** The placeholder widget which holds the category tab strip. */
	TSharedPtr<SBox> CategoryTabsContainer;

	/** The placeholder widget which holds the detail view for the currently selected content source. */
	TSharedPtr<SBox> ContentSourceDetailContainer;

	/** The content sources which the user has selected for addition to the project. */
	TArray<TSharedPtr<IContentSource>> ContentSourcesToAdd;

	/** The search box */
	TSharedPtr<class SSearchBox> SearchBoxPtr;
};
