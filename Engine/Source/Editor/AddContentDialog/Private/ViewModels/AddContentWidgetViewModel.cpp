// Copyright Epic Games, Inc. All Rights Reserved.

#include "ViewModels/AddContentWidgetViewModel.h"

#include "Containers/Set.h"
#include "Containers/UnrealString.h"
#include "ContentSourceProviderManager.h"
#include "HAL/PlatformCrt.h"
#include "IAddContentDialogModule.h"
#include "IContentSource.h"
#include "IContentSourceProvider.h"
#include "Modules/ModuleManager.h"
#include "ViewModels/CategoryViewModel.h"
#include "ViewModels/ContentSourceViewModel.h"


FAddContentWidgetViewModel::FAddContentWidgetViewModel()
{
}

TSharedRef<FAddContentWidgetViewModel> FAddContentWidgetViewModel::CreateShared()
{
	TSharedPtr<FAddContentWidgetViewModel> Shared = MakeShareable(new FAddContentWidgetViewModel());
	Shared->Initialize();
	return Shared.ToSharedRef();
}

void FAddContentWidgetViewModel::Initialize()
{
	ContentSourceFilter = MakeShared<ContentSourceTextFilter>(
		ContentSourceTextFilter::FItemToStringArray::CreateSP(this, &FAddContentWidgetViewModel::TransformContentSourceToStrings)
	);

	IAddContentDialogModule& AddContentDialogModule = FModuleManager::LoadModuleChecked<IAddContentDialogModule>("AddContentDialog");

	for (const TSharedRef<IContentSourceProvider>& ContentSourceProvider : *AddContentDialogModule.GetContentSourceProviderManager()->GetContentSourceProviders())
	{
		ContentSourceProviders.Add(ContentSourceProvider);
		ContentSourceProvider->SetContentSourcesChanged(FOnContentSourcesChanged::CreateSP(this, &FAddContentWidgetViewModel::ContentSourcesChanged));
	}

	BuildContentSourceViewModels();
}

const TArray<FCategoryViewModel>& FAddContentWidgetViewModel::GetCategories() const
{
	return Categories;
}

void FAddContentWidgetViewModel::SetOnCategoriesChanged(FOnCategoriesChanged OnCategoriesChangedIn)
{
	OnCategoriesChanged = OnCategoriesChangedIn;
}

FCategoryViewModel FAddContentWidgetViewModel::GetSelectedCategory()
{
	return SelectedCategory;
}

void FAddContentWidgetViewModel::SetSelectedCategory(FCategoryViewModel SelectedCategoryIn)
{
	SelectedCategory = SelectedCategoryIn;
	UpdateFilteredContentSourcesAndSelection(true);
	OnSelectedContentSourceChanged.ExecuteIfBound();
}

void FAddContentWidgetViewModel::SetSearchText(FText SearchTextIn)
{
	SearchText = SearchTextIn;
	ContentSourceFilter->SetRawFilterText(SearchTextIn);
	UpdateFilteredContentSourcesAndSelection(true);
}

FText FAddContentWidgetViewModel::GetSearchErrorText() const
{
	return ContentSourceFilter->GetFilterErrorText();
}

const TArray<TSharedPtr<FContentSourceViewModel>>* FAddContentWidgetViewModel::GetContentSources()
{
	return &FilteredContentSourceViewModels;
}

void FAddContentWidgetViewModel::SetOnContentSourcesChanged(FOnContentSourcesChanged OnContentSourcesChangedIn)
{
	OnContentSourcesChanged = OnContentSourcesChangedIn;
}

TSharedPtr<FContentSourceViewModel> FAddContentWidgetViewModel::GetSelectedContentSource()
{
	TSharedPtr<FContentSourceViewModel>* SelectedContentSource = CategoryToSelectedContentSourceMap.Find(SelectedCategory);
	if (SelectedContentSource != nullptr)
	{
		return *SelectedContentSource;
	}
	return TSharedPtr<FContentSourceViewModel>();
}

void FAddContentWidgetViewModel::SetSelectedContentSource(TSharedPtr<FContentSourceViewModel> SelectedContentSourceIn)
{
	// Ignore selecting the currently selected item.
	TSharedPtr<FContentSourceViewModel> SelectedContentSource = GetSelectedContentSource();
	if (SelectedContentSource != SelectedContentSourceIn)
	{
		CategoryToSelectedContentSourceMap.Add(SelectedCategory) = SelectedContentSourceIn;
		OnSelectedContentSourceChanged.ExecuteIfBound();
	}
}

void FAddContentWidgetViewModel::SetOnSelectedContentSourceChanged(FOnSelectedContentSourceChanged OnSelectedContentSourceChangedIn)
{
	OnSelectedContentSourceChanged = OnSelectedContentSourceChangedIn;
}

void FAddContentWidgetViewModel::BuildContentSourceViewModels()
{
	// Prevent reentrancy if the model sends a change event while we're getting the content sources
	if (bBuildingContentSourceViewModels)
	{
		return;
	}

	TGuardValue GuardValue(bBuildingContentSourceViewModels, true);

	Categories.Empty();
	ContentSourceViewModels.Empty();
	FilteredContentSourceViewModels.Empty();
	CategoryToSelectedContentSourceMap.Empty();
	
	// List of categories we don't want to see
	TArray<EContentSourceCategory> FilteredCategories;
	FilteredCategories.Add(EContentSourceCategory::SharedPack);
	FilteredCategories.Add(EContentSourceCategory::Unknown);

	TSet<EContentSourceCategory> FoundCategories;

	for (const TSharedPtr<IContentSourceProvider>& ContentSourceProvider : ContentSourceProviders)
	{
		for (const TSharedRef<IContentSource>& ContentSource : ContentSourceProvider->GetContentSources())
		{
			// Check if we want to see this content source - true unless all its categories are filtered out
			bool bAnyVisible = false;
			for (EContentSourceCategory ContentCategory : ContentSource->GetCategories())
			{
				if (!FilteredCategories.Contains(ContentCategory))
				{
					FoundCategories.Add(ContentCategory);
					bAnyVisible = true;
				}
			}

			if (bAnyVisible)
			{
				ContentSourceViewModels.Add(MakeShared<FContentSourceViewModel>(ContentSource));
			}
		}
	}

	for (EContentSourceCategory Found : FoundCategories)
	{
		Categories.Add(FCategoryViewModel(Found));
	}

	Categories.Sort();

	// Update the current selection for all categories.  Do this in reverse order so that the first category
	// remains selected when finished.
	for (int i = Categories.Num() - 1; i >= 0; i--)
	{
		SelectedCategory = Categories[i];
		UpdateFilteredContentSourcesAndSelection(false);
	}

	OnCategoriesChanged.ExecuteIfBound();
}

void FAddContentWidgetViewModel::UpdateFilteredContentSourcesAndSelection(bool bAllowEmptySelection)
{
	FilteredContentSourceViewModels.Empty();
	for (const TSharedPtr<FContentSourceViewModel>& ContentSource : ContentSourceViewModels)
	{
		if (ContentSource->GetCategories().Contains(SelectedCategory) && ContentSourceFilter->PassesFilter(ContentSource))
		{
			FilteredContentSourceViewModels.Add(ContentSource);
		}
	}
	OnContentSourcesChanged.ExecuteIfBound();

	if (FilteredContentSourceViewModels.Contains(GetSelectedContentSource()) == false)
	{
		TSharedPtr<FContentSourceViewModel> NewSelectedContentSource;
		if (bAllowEmptySelection == false && FilteredContentSourceViewModels.Num() > 0)
		{
			NewSelectedContentSource =  FilteredContentSourceViewModels[0];
		}
		SetSelectedContentSource(NewSelectedContentSource);
	}
}

void FAddContentWidgetViewModel::TransformContentSourceToStrings(TSharedPtr<FContentSourceViewModel> Item, OUT TArray<FString>& Array)
{
	Array.Add(Item->GetName().ToString());
}

void FAddContentWidgetViewModel::ContentSourcesChanged()
{
	BuildContentSourceViewModels();
}
