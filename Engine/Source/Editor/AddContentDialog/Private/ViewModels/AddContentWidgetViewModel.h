// Copyright Epic Games, Inc. All Rights Reserved.

#pragma once

#include "Containers/Array.h"
#include "Containers/Map.h"
#include "Delegates/Delegate.h"
#include "Internationalization/Text.h"
#include "Misc/TextFilter.h"
#include "Templates/SharedPointer.h"
#include "ViewModels/CategoryViewModel.h"

class FContentSourceViewModel;
class FString;
class IContentSourceProvider;

/** The view model for the SAddContentWidget. */
class FAddContentWidgetViewModel : public TSharedFromThis<FAddContentWidgetViewModel>
{
public:
	DECLARE_DELEGATE(FOnCategoriesChanged)
	DECLARE_DELEGATE(FOnContentSourcesChanged);
	DECLARE_DELEGATE(FOnSelectedContentSourceChanged);

	typedef TTextFilter<TSharedPtr<FContentSourceViewModel>> ContentSourceTextFilter;

	/** Creates a shared reference to a new view model. */
	static TSharedRef<FAddContentWidgetViewModel> CreateShared();

	/** Gets the view models for the current set of content source categories. */
	const TArray<FCategoryViewModel>& GetCategories() const;

	/** Sets the delegate which should be executed when the set of categories changes. */
	void SetOnCategoriesChanged(FOnCategoriesChanged OnCategoriesChangedIn);

	/** Gets the view model for the currently selected category.  Only content sources which match
		the selected category will be returned by GetContentSources(). */
	FCategoryViewModel GetSelectedCategory();

	/** Sets the currently selected category view model. Only content sources which match the selected
		category will be returned by GetContentSources(). */
	void SetSelectedCategory(FCategoryViewModel SelectedCategoryIn);

	/** Sets search text which should be used to filter the content sources. */
	void SetSearchText(FText SearchTextIn);

	/** Get any error information that was reported the last time SetSearchText was called. */
	FText GetSearchErrorText() const;

	/** Gets a filtered array of content sources which match both the selected category and the search
		text if it has been set. */
	const TArray<TSharedPtr<FContentSourceViewModel>>* GetContentSources();

	/** Sets the delegate which should be executed when the current set of content sources returned by
		GetContentSources changes. */
	void SetOnContentSourcesChanged(FOnContentSourcesChanged OnContentSourcesChangedIn);

	/** Gets the currently selected content source. */
	TSharedPtr<FContentSourceViewModel> GetSelectedContentSource();

	/** Sets the currently selected content source. */
	void SetSelectedContentSource(TSharedPtr<FContentSourceViewModel> SelectedContentSourceIn);

	/** Sets the delegate which should be executed when the selected content source changes. */
	void SetOnSelectedContentSourceChanged(FOnSelectedContentSourceChanged OnSelectedContentSourceChangedIn);

private:
	FAddContentWidgetViewModel();
	void Initialize();

	/** Builds view models for all available content sources. */
	void BuildContentSourceViewModels();

	/** Filters the current set of content sources based on the selected category and the search text, and updates the selection.
	  * @param bAllowEmptySelection Determines whether or not to select the first available item after filtering if the selected item is null, or not in the filtered set. */
	void UpdateFilteredContentSourcesAndSelection(bool bAllowEmptySelection);

	/** Converts a content source item to an array of strings for processing by the TTextFilter. */
	void TransformContentSourceToStrings(TSharedPtr<FContentSourceViewModel> Item, OUT TArray< FString >& Array);

	/** Handles notification from the content source providers when their content source arrays change. */
	void ContentSourcesChanged();

private:
	/** The array of content source providers who's content sources are being displayed. */
	TArray<TSharedPtr<IContentSourceProvider>> ContentSourceProviders;

	/** The view models for the available categories. */
	TArray<FCategoryViewModel> Categories;

	/** A combined array of all content sources from all content source providers. */
	TArray<TSharedPtr<FContentSourceViewModel>> ContentSourceViewModels;

	/** A filtered array of content sources based on the currently selected category and the search text. */
	TArray<TSharedPtr<FContentSourceViewModel>> FilteredContentSourceViewModels;

	/** A map which keeps track of the currently selected content source for each category. */
	TMap<FCategoryViewModel, TSharedPtr<FContentSourceViewModel>> CategoryToSelectedContentSourceMap;

	/** The view model for the currently selected category. */
	FCategoryViewModel SelectedCategory;

	/** The current search text. */
	FText SearchText;

	/** True while building context source view models. */
	bool bBuildingContentSourceViewModels = false;

	/** The delegate which is executed when the available categories change. */
	FOnCategoriesChanged OnCategoriesChanged;

	/** The delegate which is executed when the filtered content sources change. */
	FOnContentSourcesChanged OnContentSourcesChanged;

	/** The delegate which is executed when the currently selected content source changes. */
	FOnSelectedContentSourceChanged OnSelectedContentSourceChanged;

	/** The filter which is used to filter the content sources based on the search text. */
	TSharedPtr<ContentSourceTextFilter> ContentSourceFilter;
};
