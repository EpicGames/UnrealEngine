// Copyright Epic Games, Inc. All Rights Reserved.

#include "ViewModels/CategoryViewModel.h"

#include "IContentSource.h"
#include "Internationalization/Internationalization.h"

#define LOCTEXT_NAMESPACE "ContentSourceViewModel"

FCategoryViewModel::FCategoryViewModel()
{
	Category = EContentSourceCategory::Unknown;
	Initialize();
}

FCategoryViewModel::FCategoryViewModel(EContentSourceCategory InCategory)
{
	Category = InCategory;
	Initialize();
}

void FCategoryViewModel::Initialize()
{
	switch (Category)
	{
	case EContentSourceCategory::BlueprintFeature:
		Text = LOCTEXT("BlueprintFeature", "Blueprint");
		SortID = 0;
		break;
	case EContentSourceCategory::CodeFeature:
		Text = LOCTEXT("CodeFeature", "C++");
		SortID = 1;
		break;
	case EContentSourceCategory::EnterpriseFeature:
		Text = LOCTEXT("EnterpriseFeature", "Unreal Studio Feature");
		SortID = 2;
		break;
	case EContentSourceCategory::Content:
		Text = LOCTEXT("ContentPacks", "Content");
		SortID = 3;
		break;
	case EContentSourceCategory::EnterpriseContent:
		Text = LOCTEXT("EnterpriseContentPacks", "Unreal Studio Content");
		SortID = 4;
		break;
	default:
		Text = LOCTEXT("Miscellaneous", "Miscellaneous");
		SortID = 5;
		break;
	}
}

#undef LOCTEXT_NAMESPACE
