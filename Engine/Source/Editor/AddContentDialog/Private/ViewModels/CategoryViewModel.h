// Copyright Epic Games, Inc. All Rights Reserved.

#pragma once


#include "HAL/Platform.h"
#include "Internationalization/Text.h"

enum class EContentSourceCategory:uint8;

/** A view model for displaying a content source category in the UI. */
class FCategoryViewModel
{
public:
	FCategoryViewModel();

	FCategoryViewModel(EContentSourceCategory InCategory);

	/** Gets the display name of the category. */
	const FText& GetText() const { return Text; }

	inline bool operator==(const FCategoryViewModel& Other) const
	{
		return Category == Other.Category;
	}

	bool operator<(FCategoryViewModel const& Other) const
	{
		return SortID < Other.SortID;
	}

	uint32 GetTypeHash() const { return (uint32) Category; }

private:
	void Initialize();

private:
	EContentSourceCategory Category;
	FText Text;
	int SortID;
};

inline uint32 GetTypeHash(const FCategoryViewModel& CategoryViewModel)
{
	return CategoryViewModel.GetTypeHash();
}
