// Copyright Epic Games, Inc. All Rights Reserved.

#include "ViewModels/ContentSourceViewModel.h"

#include "Brushes/SlateDynamicImageBrush.h"
#include "Containers/StringView.h"
#include "HAL/PlatformCrt.h"
#include "IContentSource.h"
#include "IImageWrapper.h"
#include "IImageWrapperModule.h"
#include "Internationalization/Culture.h"
#include "Internationalization/Internationalization.h"
#include "Math/Vector2D.h"
#include "Modules/ModuleManager.h"
#include "UObject/NameTypes.h"
#include "ViewModels/CategoryViewModel.h"

struct FSlateBrush;


#define LOCTEXT_NAMESPACE "ContentSourceViewModel"

uint32 FContentSourceViewModel::ImageID = 0;


FContentSourceViewModel::FContentSourceViewModel(TSharedPtr<IContentSource> ContentSourceIn)
{
	ContentSource = ContentSourceIn;
	SetupBrushes();

	for (EContentSourceCategory Category : ContentSource->GetCategories())
	{
		Categories.Add(FCategoryViewModel(Category));
	}
}

const TSharedPtr<IContentSource>& FContentSourceViewModel::GetContentSource() const
{
	return ContentSource;
}

const FText& FContentSourceViewModel::GetName() const
{
	const FString& CurrentLanguage = FInternationalization::Get().GetCurrentLanguage()->GetName();
	if (CachedNameText.Language != CurrentLanguage)
	{
		CachedNameText =
			FCachedContentText
		{
			CurrentLanguage,
			ChooseLocalizedText(ContentSource->GetLocalizedNames(), CurrentLanguage)
		};
	}
	return CachedNameText.Text;
}

const FText& FContentSourceViewModel::GetDescription() const
{
	const FString& CurrentLanguage = FInternationalization::Get().GetCurrentLanguage()->GetName();
	if (CachedDescriptionText.Language != CurrentLanguage)
	{
		CachedDescriptionText = 
			FCachedContentText 
			{ 
				CurrentLanguage, 
				ChooseLocalizedText(ContentSource->GetLocalizedDescriptions(), CurrentLanguage) 
			};
	}
	return CachedDescriptionText.Text;
}

const FText& FContentSourceViewModel::GetAssetTypes() const
{
	const FString& CurrentLanguage = FInternationalization::Get().GetCurrentLanguage()->GetName();
	if (CachedAssetTypeText.Language != CurrentLanguage)
	{
		CachedAssetTypeText =
			FCachedContentText
		{
			CurrentLanguage,
			ChooseLocalizedText(ContentSource->GetLocalizedAssetTypes(), CurrentLanguage)
		};
	}
	return CachedAssetTypeText.Text;
}

FStringView FContentSourceViewModel::GetClassTypes() const
{
	return ContentSource->GetClassTypesUsed();
}

const TArray<FCategoryViewModel>& FContentSourceViewModel::GetCategories() const
{
	return Categories;
}

const TSharedPtr<FSlateBrush>& FContentSourceViewModel::GetIconBrush() const
{
	return IconBrush;
}

const TArray<TSharedPtr<FSlateBrush>>& FContentSourceViewModel::GetScreenshotBrushes() const
{
	return ScreenshotBrushes;
}

void FContentSourceViewModel::SetupBrushes()
{
	if (ContentSource->GetIconData().IsValid())
	{
		const FString IconBrushName = GetName().ToString() + "_" + ContentSource->GetIconData()->GetName();
		IconBrush = CreateBrushFromRawData(IconBrushName, *ContentSource->GetIconData()->GetData());
	}

	for (const TSharedPtr<FImageData>& ScreenshotData : ContentSource->GetScreenshotData())
	{
		if (ScreenshotData.IsValid() == true)
		{
			const FString ScreenshotBrushName = GetName().ToString() + "_" + ScreenshotData->GetName();
			ScreenshotBrushes.Add(CreateBrushFromRawData(ScreenshotBrushName, *ScreenshotData->GetData()));
	    }
    }
}

TSharedPtr<FSlateDynamicImageBrush> FContentSourceViewModel::CreateBrushFromRawData(const FString& ResourceNamePrefix, const TArray<uint8>& RawData) const
{
	TSharedPtr< FSlateDynamicImageBrush > Brush;

	uint32 BytesPerPixel = 4;
	int32 Width = 0;
	int32 Height = 0;

	bool bSucceeded = false;
	TArray<uint8> DecodedImage;
	IImageWrapperModule& ImageWrapperModule = FModuleManager::LoadModuleChecked<IImageWrapperModule>(FName("ImageWrapper"));
	TSharedPtr<IImageWrapper> ImageWrapper = ImageWrapperModule.CreateImageWrapper(EImageFormat::PNG);
	if (ImageWrapper.IsValid() && (RawData.Num() > 0) && ImageWrapper->SetCompressed(RawData.GetData(), RawData.Num()))
	{
		Width = ImageWrapper->GetWidth();
		Height = ImageWrapper->GetHeight();

		if (ImageWrapper->GetRaw(ERGBFormat::BGRA, 8, DecodedImage))
		{
			bSucceeded = true;
		}
	}

	if (bSucceeded)
	{
		FString UniqueResourceName = ResourceNamePrefix + "_" + FString::FromInt(ImageID++);
		Brush = FSlateDynamicImageBrush::CreateWithImageData(FName(*UniqueResourceName), FVector2D((float)ImageWrapper->GetWidth(), (float)ImageWrapper->GetHeight()), DecodedImage);
	}

	return Brush;
}

FText FContentSourceViewModel::ChooseLocalizedText(const TArray<FLocalizedText>& Choices, const FString& InCurrentLanguage) const
{
	auto FindLocalizedTextForCulture = [&Choices](const FString& InCulture)
	{
		return Choices.FindByPredicate([&InCulture](const FLocalizedText& LocalizedText)
		{
			return InCulture == LocalizedText.GetTwoLetterLanguage();
		});
	};

	// Try and find a prioritized localized translation
	{
		const TArray<FString> PrioritizedCultureNames = FInternationalization::Get().GetPrioritizedCultureNames(InCurrentLanguage);
		for (const FString& CultureName : PrioritizedCultureNames)
		{
			if (const FLocalizedText* LocalizedTextForCulture = FindLocalizedTextForCulture(CultureName))
			{
				return LocalizedTextForCulture->GetText();
			}
		}
	}

	// We failed to find a localized translation, see if we have English text available to use
	if (InCurrentLanguage != TEXT("en"))
	{
		if (const FLocalizedText* LocalizedTextForEnglish = FindLocalizedTextForCulture(TEXT("en")))
		{
			return LocalizedTextForEnglish->GetText();
		}
	}

	// We failed to find English, see if we have any translations available to use
	if (Choices.Num() > 0)
	{
		return Choices[0].GetText();
	}

	return FText();
}

#undef LOCTEXT_NAMESPACE
