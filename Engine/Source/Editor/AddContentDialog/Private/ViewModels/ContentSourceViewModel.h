// Copyright Epic Games, Inc. All Rights Reserved.

#pragma once

#include "Containers/Array.h"
#include "Containers/StringFwd.h"
#include "Containers/UnrealString.h"
#include "HAL/Platform.h"
#include "Internationalization/Text.h"
#include "Templates/SharedPointer.h"
#include "ViewModels/CategoryViewModel.h"

class FLocalizedText;
class IContentSource;
struct FSlateBrush;
struct FSlateDynamicImageBrush;

/** A view model for displaying and interacting with an IContentSource in the FAddContentDialog. */
class FContentSourceViewModel : public TSharedFromThis<FContentSourceViewModel>
{
public:
	/** Creates a view model for a supplied content source. */
	FContentSourceViewModel(TSharedPtr<IContentSource> ContentSourceIn);

	/** Gets the content source represented by this view model. */
	const TSharedPtr<IContentSource>& GetContentSource() const;

	/** Gets the display name for this content source. */
	const FText& GetName() const;

	/** Gets the description of this content source. */
	const FText& GetDescription() const;

	/** Gets the asset types used in this content source. */
	const FText& GetAssetTypes() const;

	/** Gets the class types used in this content source. */
	FStringView GetClassTypes() const;

	/** Gets the view models for the categories for this content source. */
	const TArray<FCategoryViewModel>& GetCategories() const;

	/** Gets the brush which should be used to draw the icon representation of this content source. */
	const TSharedPtr<FSlateBrush>& GetIconBrush() const;

	/** Gets an array or brushes which should be used to display screenshots for this content source. */
	const TArray<TSharedPtr<FSlateBrush>>& GetScreenshotBrushes() const;

private:
	/** Sets up brushes from the images data supplied by the IContentSource. */
	void SetupBrushes();

	/** Creates a slate brush from raw binary PNG formatted image data and the supplied prefix. */
	TSharedPtr<FSlateDynamicImageBrush> CreateBrushFromRawData(const FString& ResourceNamePrefix, const TArray<uint8>& RawData) const;

	/** Selects the text from an array which matches the given language. */
	FText ChooseLocalizedText(const TArray<FLocalizedText>& Choices, const FString& InCurrentLanguage) const;

private:
	struct FCachedContentText
	{
		FString Language;
		FText Text;
	};

	/** The content source represented by this view model. */
	TSharedPtr<IContentSource> ContentSource;

	/** The brush which should be used to draw the icon representation of this content source. */
	TSharedPtr<FSlateBrush> IconBrush;

	/** An array or brushes which should be used to display screenshots for this content source. */
	TArray<TSharedPtr<FSlateBrush>> ScreenshotBrushes;

	/** The view models for the categories for this content source. */
	TArray<FCategoryViewModel> Categories;

	/** The information used/returned the last time the name of the content source was requested. */
	mutable FCachedContentText CachedNameText;

	/** The information used/returned the last time the description of the content source was requested. */
	mutable FCachedContentText CachedDescriptionText;

	/** The information used/returned the last time the asset types of the content source was requested. */
	mutable FCachedContentText CachedAssetTypeText;

	/** Keeps track of a unique increasing id which is appended to each brush name.  This avoids an issue
		where two brushes are created with the same name, and then both brushes texture data gets deleted
		when either brush is destructed. */
	static uint32 ImageID;
};
