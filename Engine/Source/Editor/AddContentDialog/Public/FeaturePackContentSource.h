// Copyright Epic Games, Inc. All Rights Reserved.

#pragma once

#include "Containers/Array.h"
#include "Containers/UnrealString.h"
#include "CoreMinimal.h"
#include "CoreTypes.h"
#include "Editor/AddContentDialog/Private/IContentSource.h"
#include "HAL/PlatformCrt.h"
#include "Internationalization/Text.h"
#include "Misc/AssertionMacros.h"
#include "Templates/SharedPointer.h"
#include "UObject/Class.h"
#include "UObject/ObjectMacros.h"

#include "FeaturePackContentSource.generated.h"

#define UE_API ADDCONTENTDIALOG_API

class FJsonValue;
class FPakPlatformFile;
class UObject;
struct FSearchEntry;

struct FPackData
{
	FString PackSource;
	FString PackName;
	FString PackMap;
	TArray<UObject*>	ImportedObjects;
};

class FLocalizedTextArray
{
public:
	FLocalizedTextArray()
	{
	}

	/** Creates a new FLocalizedText
		@param InTwoLetterLanguage - The iso 2-letter language specifier.
		@param InText - The text in the language specified */
	FLocalizedTextArray(FString InTwoLetterLanguage, FString InText)
	{
		TwoLetterLanguage = InTwoLetterLanguage;
		TArray<FString> AsArray;
		InText.ParseIntoArray(AsArray,TEXT(","));
		for (int32 iString = 0; iString < AsArray.Num() ; iString++)
		{
			Tags.Add(FText::FromString(AsArray[iString]));
		}
	}

	/** Gets the iso 2-letter language specifier for this text. */
	const FString& GetTwoLetterLanguage() const
	{
		return TwoLetterLanguage;
	}

	/** Gets the array of tags in the language specified. */
	const TArray<FText>& GetTags() const
	{
		return Tags;
	}

private:
	FString TwoLetterLanguage;
	TArray<FText> Tags;
};


/** Defines categories for shared template resource levels. */
UENUM()
enum class EFeaturePackDetailLevel :uint8
{
	Standard,
	High,
};

/* Structure that defines a shared feature pack resource. */
USTRUCT()
struct FFeaturePackLevelSet
{
	GENERATED_BODY()

	FFeaturePackLevelSet(){};

	/** Creates a new FFeaturePackLevelSet
		@param InMountName - Name of the pack/folder to insert to 
		@param InDetailLevels - The levels available for this pack*/
	FFeaturePackLevelSet(FString InMountName, TArray<EFeaturePackDetailLevel> InDetailLevels)
	{
		MountName = InMountName;
		DetailLevels = InDetailLevels;
	}

	/* List of shared resource levels for this shared resource.*/
	UPROPERTY()
	TArray<EFeaturePackDetailLevel> DetailLevels;

	/* Mount name for the shared resource - this is the folder the resource will be copied to on project generation as well as the name of the folder that will appear in the content browser. */
	UPROPERTY()
	FString MountName;

	FString GetFeaturePackNameForLevel(EFeaturePackDetailLevel InLevel, bool bLevelRequired = false)
	{
		check(DetailLevels.Num()>0); // We need at least one detail level defined
		int32 Index = DetailLevels.Find(InLevel);
		FString DetailString;
		if( Index != INDEX_NONE)
		{			
			UEnum::GetValueAsString(TEXT("/Script/AddContentDialog.EFeaturePackDetailLevel"), InLevel, DetailString);					
		}
		else 
		{
			check(bLevelRequired==false); // The level is REQUIRED and we don't have it !
			// If we didn't have the requested level, use the first
			UEnum::GetValueAsString(TEXT("/Script/AddContentDialog.EFeaturePackDetailLevel"), DetailLevels[0], DetailString);
		}
		FString NameString = MountName+DetailString + TEXT(".upack");
		return NameString;
	}
};

/* Structure that defines a shared feature pack resource. */
USTRUCT()
struct FFeatureAdditionalFiles
{
	GENERATED_BODY()

	FFeatureAdditionalFiles(){};
	
	/* Name of the folder to insert the files to */
	UPROPERTY()
	FString DestinationFilesFolder;

	/* List of files to insert */
	UPROPERTY()
	TArray<FString> AdditionalFilesList;
};

class FPakPlatformFile;
struct FSearchEntry;

/** A content source which represents a content upack. */
class FFeaturePackContentSource : public IContentSource
{
public:
	UE_API FFeaturePackContentSource();
	UE_API FFeaturePackContentSource(FString InFeaturePackPath);

	UE_API virtual ~FFeaturePackContentSource();

	UE_API virtual const TArray<FLocalizedText>& GetLocalizedNames() const override;
	UE_API virtual const TArray<FLocalizedText>& GetLocalizedDescriptions() const override;
	
	UE_API virtual const TArray<EContentSourceCategory>& GetCategories() const override;
	UE_API virtual const TArray<FLocalizedText>& GetLocalizedAssetTypes() const override;
	UE_API virtual const FString& GetSortKey() const override;
	UE_API virtual const FString& GetClassTypesUsed() const override;
	UE_API virtual const FString& GetIdent() const override;
	UE_API virtual TSharedPtr<FImageData> GetIconData() const override;
	UE_API virtual const TArray<TSharedPtr<FImageData>>& GetScreenshotData() const override;
	UE_API const FString& GetFocusAssetName() const;

	UE_API virtual bool InstallToProject(FString InstallPath) override;

	UE_API void InsertAdditionalFeaturePacks();
	UE_API bool InsertAdditionalResources(TArray<FFeaturePackLevelSet> InAdditionalFeaturePacks,EFeaturePackDetailLevel RequiredLevel, const FString& InDestinationFolder,TArray<FString>& InFilesCopied);

	UE_API virtual bool IsDataValid() const override;
			
	/*
	 * Copies the list of files specified in 'AdditionFilesToInclude' section in the config.ini of the feature pack.
	 *
	 * @param DestinationFolder	Destination folder for the files
	 * @param FilesCopied		List of files copied
	 * @param bContainsSource 	Set to true if the file list contains any source files
	 * @returns true if config file was read and parsed successfully
	 */
	UE_API void CopyAdditionalFilesToFolder( const FString& DestinationFolder, TArray<FString>& FilesCopied, bool &bHasSourceFiles, FString InGameFolder = FString() );

	/*
	 * Returns a list of additional files (including the path) as specified in the config file if one exists in the pack file.
	 *
	 * @param FileList		  array to receive list of files
	 * @param bContainsSource did the file list contain any source files
	 * @returns true if config file was read and parsed successfully
	 */
	UE_API bool GetAdditionalFilesForPack(TArray<FString>& FileList, bool& bContainsSource);

	static UE_API void ImportPendingPacks();
	
	/* Errors found when parsing manifest (if any) */
	TArray<FString>	ParseErrors;
	
	UE_API void BuildListOfAdditionalFiles(TArray<FString>& AdditionalFileSourceList,TArray<FString>& FileList, bool& bContainsSourceFiles);

private:
	static UE_API void ParseAndImportPacks();
	UE_API bool LoadPakFileToBuffer(FPakPlatformFile& PakPlatformFile, FString Path, TArray<uint8>& Buffer);
	
	
	/*
	 * Extract the list of additional files defined in config file to an array
	 *
	 * @param ConfigFile	  config file as a string
	 * @param FileList		  array to receive list of files
	 * @param bContainsSource did the file list contain any source files
	 */
	UE_API bool ExtractListOfAdditionalFiles(const FString& ConfigFile, TArray<FString>& FileList,bool& bContainsSource);

	UE_API void RecordAndLogError(const FString& ErrorString);

	/* Load the images for the icon and screen shots directly from disk */
	UE_API bool LoadFeaturePackImageData();
	
	/* extract the images for the icon and screen shots from a pak file */
	UE_API bool LoadFeaturePackImageDataFromPackFile(FPakPlatformFile& PakPlatformFile);
	
	/* Parse the manifest string describing this pack file */
	UE_API bool ParseManifestString(const FString& ManifestString);

	/** Selects an FLocalizedText from an array which matches either the supplied language code, or the default language code. */
	UE_API FLocalizedTextArray ChooseLocalizedTextArray(TArray<FLocalizedTextArray> Choices, FString LanguageCode);
	UE_API FLocalizedText ChooseLocalizedText(TArray<FLocalizedText> Choices, FString LanguageCode);

	/* The path of the file we used to create this feature pack instance */
	FString FeaturePackPath;
	
	/* Array of localised names */
	TArray<FLocalizedText> LocalizedNames;
	
	/* Array of localised descriptions */
	TArray<FLocalizedText> LocalizedDescriptions;
	
	/* Defines the type(s) of feature pack this is */
	TArray<EContentSourceCategory> Categories;
	
	/* Filename of the icon */
	FString IconFilename;
	
	/* Image data for the icon */
	TSharedPtr<FImageData> IconData;
	
	/* Filenames of the preview screenshots */
	TArray<TSharedPtr<FJsonValue>> ScreenshotFilenameArray;
	
	/* Image data of the preview screenshots */
	TArray<TSharedPtr<FImageData>> ScreenshotData;
	
	/* Array of localised assset type names */
	TArray<FLocalizedText> LocalizedAssetTypesList;
	
	/* Comma delimited string listing the class types */
	FString ClassTypes;
	
	/* true if the pack is valid */
	bool bPackValid;
	
	/* Asset to focus after loading the pack */
	FString FocusAssetIdent;
	
	/* Key used when sorting in the add dialog */
	FString SortKey;
	
	/* Tags searched when typing in the super search box */
	TArray<FLocalizedTextArray> LocalizedSearchTags;
	
	/* Other feature packs this pack needs (shared assets) */
	TArray<FFeaturePackLevelSet> AdditionalFeaturePacks;
		
	/* Additional files to copy when installing this pack */
	FFeatureAdditionalFiles AdditionalFilesForPack;
	
	/* Are the contents in a pack file or did we just read a manifest for the pack */
	bool bContentsInPakFile;

	/* Feature pack mount point */
	FString MountPoint;

	FString Identity;
	FString VersionNumber;
};

#undef UE_API
