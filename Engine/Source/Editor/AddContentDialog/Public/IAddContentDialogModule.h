// Copyright Epic Games, Inc. All Rights Reserved.

#pragma once

#include "CoreMinimal.h"
#include "Modules/ModuleInterface.h"
#include "Widgets/SWindow.h"

class FContentSourceProviderManager;

/** Defines methods for interacting with the Add Content Dialog.  */
class IAddContentDialogModule : public IModuleInterface
{

public:
	/** Gets the object responsible for managing content source providers. */
	virtual TSharedRef<FContentSourceProviderManager> GetContentSourceProviderManager() = 0;

	/** Creates a dialog for adding existing content to a project. */
	virtual void ShowDialog(TSharedRef<SWindow> ParentWindow) = 0;
};
